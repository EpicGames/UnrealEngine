[CoreRedirects]
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerWiggle",NewName="/Script/PropertyAnimator.PropertyAnimatorWiggle")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerWave",NewName="/Script/PropertyAnimator.PropertyAnimatorWave")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerTime",NewName="/Script/PropertyAnimator.PropertyAnimatorTime")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerOscillate",NewName="/Script/PropertyAnimator.PropertyAnimatorOscillate")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerFloatBase",NewName="/Script/PropertyAnimator.PropertyAnimatorNumericBase")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerBounce",NewName="/Script/PropertyAnimator.PropertyAnimatorBounce")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerBlink",NewName="/Script/PropertyAnimator.PropertyAnimatorPulse")
+ClassRedirects=(OldName="/Script/PropertyAnimator.PropertyAnimatorBlink",NewName="/Script/PropertyAnimator.PropertyAnimatorPulse")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerBase",NewName="/Script/PropertyAnimator.PropertyAnimatorBase")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerWaveDoubleChannelOverrideContainer",NewName="/Script/PropertyAnimator.PropertyAnimatorWaveDoubleChannelOverrideContainer")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerWaveDoubleChannelEvaluatorSystem",NewName="/Script/PropertyAnimator.PropertyAnimatorWaveDoubleChannelEvaluatorSystem")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerEasingDoubleChannelOverrideContainer",NewName="/Script/PropertyAnimator.PropertyAnimatorEasingDoubleChannelOverrideContainer")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerEasingDoubleChannelEvaluatorSystem",NewName="/Script/PropertyAnimator.PropertyAnimatorEasingDoubleChannelEvaluatorSystem")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerCustomTimeSource",NewName="/Script/PropertyAnimator.PropertyAnimatorCustomTimeSource")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerTimeSourceBase",NewName="/Script/PropertyAnimator.PropertyAnimatorTimeSourceBase")
+ClassRedirects=(OldName="/Script/PropertyController.PropertyControllerWorldTimeSource",NewName="/Script/PropertyAnimator.PropertyAnimatorWorldTimeSource")
+ClassRedirects=(OldName="/Script/AvalanchePropertyAnimator.AvaPropertyAnimatorTextResolver",NewName="/Script/PropertyAnimator.PropertyAnimatorTextResolver")
+ClassRedirects=(OldName="/Script/PropertyAnimator.PropertyAnimatorCurveWave",NewName="/Script/PropertyAnimator.PropertyAnimatorCurve")
+ClassRedirects=(OldName="/Script/PropertyAnimator.PropertyAnimatorFloatBase",NewName="/Script/PropertyAnimator.PropertyAnimatorNumericBase")

+EnumRedirects=(OldName="/Script/PropertyController.EPropertyControllerEasingType",NewName="/Script/PropertyAnimator.EPropertyAnimatorEasingType")
+EnumRedirects=(OldName="/Script/PropertyController.EPropertyControllerEasingFunction",NewName="/Script/PropertyAnimator.EPropertyAnimatorEasingFunction")
+EnumRedirects=(OldName="/Script/PropertyController.EPropertyControllerWaveFunction",NewName="/Script/PropertyAnimator.EPropertyAnimatorWaveFunction")

+StructRedirects=(OldName="/Script/PropertyController.PropertyControllerFloatPropertyOptions",NewName="/Script/PropertyAnimator.PropertyAnimatorFloatPropertyOptions")
+StructRedirects=(OldName="/Script/PropertyController.PropertyControllerEasingDoubleChannel",NewName="/Script/PropertyAnimator.PropertyAnimatorEasingDoubleChannel")
+StructRedirects=(OldName="/Script/PropertyController.PropertyControllerEasingParameters",NewName="/Script/PropertyAnimator.PropertyAnimatorEasingParameters")
+StructRedirects=(OldName="/Script/PropertyController.PropertyControllerWaveDoubleChannel",NewName="/Script/PropertyAnimator.PropertyAnimatorWaveDoubleChannel")
+StructRedirects=(OldName="/Script/PropertyController.PropertyControllerWaveParameters",NewName="/Script/PropertyAnimator.PropertyAnimatorWaveParameters")

+PropertyRedirects=(OldName="/Script/PropertyAnimator.PropertyAnimatorFloatBase.Strength",NewName="/Script/PropertyAnimator.PropertyAnimatorNumericBase.GlobalMagnitude")