{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Property Animator",
	"Description": "Re-usable behaviors to animate the value of one or more properties",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PropertyAnimator",
			"Type": "Runtime",
			"LoadingPhase": "Default",
         	"TargetDenyList": [ "Server" ]
		},
		{
			"Name": "PropertyAnimatorEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "PropertyAnimatorCore",
			"Enabled": true
		},
		{
			"Name": "Text3D",
			"Enabled": true
		},
		{
			"Name": "AudioSynesthesia",
			"Enabled": true
		}
	]
}