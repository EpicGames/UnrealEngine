// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorBounce.h"

#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "PropertyAnimatorShared.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorBounce"

void UPropertyAnimatorBounce::SetInvertEffect(bool bInvert)
{
	if (bInvertEffect == bInvert)
	{
		return;
	}

	bInvertEffect = bInvert;
	OnInvertEffect();
}

void UPropertyAnimatorBounce::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Bounce");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Bounce");
}

bool UPropertyAnimatorBounce::EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
{
	const double Frequency = InParameters.GetValueDouble(FrequencyParameterName).GetValue();
	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();

	double TimeProgress = FMath::Wrap(TimeElapsed, 0.0, 1.0 / Frequency);

	TimeProgress = bInvertEffect ? TimeProgress : 1 - TimeProgress;

	const double EasingValue = UE::PropertyAnimator::Easing::Bounce(TimeProgress, EPropertyAnimatorEasingType::In);

	InParameters.AddProperty(AlphaParameterName, EPropertyBagPropertyType::Float);
	InParameters.SetValueFloat(AlphaParameterName, EasingValue);

	return InContext->EvaluateProperty(InPropertyData, InParameters, OutEvaluationResult);
}

#undef LOCTEXT_NAMESPACE
