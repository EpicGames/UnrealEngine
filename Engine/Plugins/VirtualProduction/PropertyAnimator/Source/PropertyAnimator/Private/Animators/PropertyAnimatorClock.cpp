// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorClock.h"

#include "Misc/DateTime.h"
#include "Properties/Converters/PropertyAnimatorCoreConverterBase.h"
#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorClock"

namespace UE::Private::Animator
{
	static TMap<TCHAR, TFunction<FString(const FDateTime&)>> FormatFunctions;
}

void UPropertyAnimatorClock::RegisterFormat(const TCHAR InChar, TFunction<FString(const FDateTime&)> InFormatter)
{
	UE::Private::Animator::FormatFunctions.Add(InChar, InFormatter);
}

void UPropertyAnimatorClock::UnregisterFormat(const TCHAR InChar)
{
	UE::Private::Animator::FormatFunctions.Remove(InChar);
}

UPropertyAnimatorClock::UPropertyAnimatorClock()
{
	if (IsTemplate())
	{
		RegisterFormat(TEXT('a'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%a")); });
		RegisterFormat(TEXT('A'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%A")); });
		RegisterFormat(TEXT('w'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%w")); });
		RegisterFormat(TEXT('y'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%y")); });
		RegisterFormat(TEXT('Y'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%Y")); });
		RegisterFormat(TEXT('b'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%b")); });
		RegisterFormat(TEXT('B'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%B")); });
		RegisterFormat(TEXT('m'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%m")); });
		RegisterFormat(TEXT('n'), [](const FDateTime& InDateTime)->FString{ return FString::FromInt(InDateTime.GetMonth()); });
		RegisterFormat(TEXT('d'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%d")); });
		RegisterFormat(TEXT('e'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%e")); });
		RegisterFormat(TEXT('j'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%j")); });
		RegisterFormat(TEXT('J'), [](const FDateTime& InDateTime)->FString{ return FString::FromInt(InDateTime.GetDayOfYear()); });
		RegisterFormat(TEXT('l'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%l")); });
		RegisterFormat(TEXT('I'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%I")); });
		RegisterFormat(TEXT('H'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%H")); });
		RegisterFormat(TEXT('h'), [](const FDateTime& InDateTime)->FString{ return FString::FromInt(InDateTime.GetHour()); });
		RegisterFormat(TEXT('o'), [](const FDateTime& InDateTime)->FString{ return LexToString(InDateTime.GetTicks() / ETimespan::TicksPerHour); });
		RegisterFormat(TEXT('M'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%M")); });
		RegisterFormat(TEXT('N'), [](const FDateTime& InDateTime)->FString{ return FString::FromInt(InDateTime.GetMinute()); });
		RegisterFormat(TEXT('i'), [](const FDateTime& InDateTime)->FString{ return LexToString(InDateTime.GetTicks() / ETimespan::TicksPerMinute); });
		RegisterFormat(TEXT('S'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%S")); });
		RegisterFormat(TEXT('s'), [](const FDateTime& InDateTime)->FString{ return FString::FromInt(InDateTime.GetSecond()); });
		RegisterFormat(TEXT('c'), [](const FDateTime& InDateTime)->FString{ return LexToString(InDateTime.GetTicks() / ETimespan::TicksPerSecond); });
		RegisterFormat(TEXT('f'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToString(TEXT("%s")); });
		RegisterFormat(TEXT('F'), [](const FDateTime& InDateTime)->FString{ return FString::FromInt(InDateTime.GetMillisecond()); });
		RegisterFormat(TEXT('p'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%p")); });
		RegisterFormat(TEXT('P'), [](const FDateTime& InDateTime)->FString{ return InDateTime.ToFormattedString(TEXT("%P")); });
		RegisterFormat(TEXT('t'), [](const FDateTime& InDateTime)->FString{ return LexToString(InDateTime.GetTicks()); });
	}
}

void UPropertyAnimatorClock::SetDisplayFormat(const FString& InDisplayFormat)
{
	DisplayFormat = InDisplayFormat;
}

void UPropertyAnimatorClock::SetDisplayMask(const FString& InPadding)
{
	DisplayMask = InPadding;
}

FString UPropertyAnimatorClock::FormatDateTime(const FDateTime& InDateTime, const FString& InDisplayFormat, const FString& InMask)
{
	TStringBuilder<256> Builder;

    for (int32 CharIndex = 0; CharIndex < InDisplayFormat.Len(); ++CharIndex)
    {
        if (InDisplayFormat[CharIndex] == TEXT('%') && CharIndex + 1 < InDisplayFormat.Len())
        {
			if (const TFunction<FString(const FDateTime&)>* Formatter = UE::Private::Animator::FormatFunctions.Find(InDisplayFormat[CharIndex + 1]))
			{
				Builder.Append((*Formatter)(InDateTime));
			}

            ++CharIndex;
        }
        else
        {
        	Builder.AppendChar(InDisplayFormat[CharIndex]);
        }
    }

	if (Builder.Len() < InMask.Len())
	{
		Builder.Prepend(InMask.Left(InMask.Len() - Builder.Len()));
	}

    return FString(Builder);
}

void UPropertyAnimatorClock::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Clock");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Clock");
}

void UPropertyAnimatorClock::EvaluateProperties(FInstancedPropertyBag& InParameters)
{
	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();

	const FTimespan ElapsedTimeSpan = FTimespan::FromSeconds(TimeElapsed);
	const FDateTime DateTime(ElapsedTimeSpan > FTimespan::Zero() ? ElapsedTimeSpan.GetTicks() : 0);
	const FString FormattedDateTime = FormatDateTime(DateTime, DisplayFormat, DisplayMask);

	EvaluateEachLinkedProperty([this, FormattedDateTime](
		UPropertyAnimatorCoreContext* InContext
		, const FPropertyAnimatorCoreData& InResolvedProperty
		, FInstancedPropertyBag& InEvaluatedValues
		, int32 InRangeIndex
		, int32 InRangeMax)->bool
	{
		const FName PropertyHash = InResolvedProperty.GetLocatorPathHash();

		InEvaluatedValues.AddProperty(PropertyHash, EPropertyBagPropertyType::String);
		InEvaluatedValues.SetValueString(PropertyHash, FormattedDateTime);

		return true;
	});
}

bool UPropertyAnimatorClock::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InValue->AsMutableObject();

		FString DisplayFormatValue = DisplayFormat;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorClock, DisplayFormat), DisplayFormatValue);
		SetDisplayFormat(DisplayFormatValue);

		FString DisplayMaskValue = DisplayMask;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorClock, DisplayMask), DisplayMaskValue);
		SetDisplayMask(DisplayMaskValue);

		return true;
	}

	return false;
}

bool UPropertyAnimatorClock::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = OutValue->AsMutableObject();

		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorClock, DisplayFormat), DisplayFormat);
		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorClock, DisplayMask), DisplayMask);

		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
