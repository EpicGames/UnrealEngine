// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorCounter.h"

#include "Properties/Converters/PropertyAnimatorCoreConverterBase.h"
#include "Settings/PropertyAnimatorSettings.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCounter"

void FPropertyAnimatorCounterFormat::EnsureCharactersLength()
{
	if (DecimalCharacter.Len() > 1)
	{
		DecimalCharacter.RemoveAt(1, DecimalCharacter.Len() - 1);
	}

	if (PaddingCharacter.Len() > 1)
	{
		PaddingCharacter.RemoveAt(1, PaddingCharacter.Len() - 1);
	}

	if (GroupingCharacter.Len() > 1)
	{
		GroupingCharacter.RemoveAt(1, GroupingCharacter.Len() - 1);
	}
}

FString FPropertyAnimatorCounterFormat::FormatNumber(double InNumber) const
{
	const bool bPositive = InNumber >= 0;
	InNumber = FMath::Abs(InNumber);

	if (RoundingMode != EPropertyAnimatorCounterRoundingMode::None)
	{
		switch (RoundingMode)
		{
		case EPropertyAnimatorCounterRoundingMode::Round:
			InNumber = FMath::RoundToDouble(InNumber);
		break;
		case EPropertyAnimatorCounterRoundingMode::Floor:
			InNumber = FMath::FloorToDouble(InNumber);
		break;
		case EPropertyAnimatorCounterRoundingMode::Ceil:
			InNumber = FMath::CeilToDouble(InNumber);
		break;
		default:
		break;
		}
	}

	const int32 IntegerCount = MinIntegerCount;
	const int32 DecimalCount = MaxDecimalCount;

	FString IntegerPart = LexToString(FMath::TruncToInt(InNumber));
	FString DecimalPart;

	if (IntegerPart.Len() > IntegerCount && bTruncate)
	{
		IntegerPart.RemoveAt(0, IntegerPart.Len() - IntegerCount);
	}

	if (DecimalCount > 0)
	{
		DecimalPart = FString::SanitizeFloat(FMath::Fractional(InNumber));

		DecimalPart.RemoveFromStart(TEXT("0."));

		if (DecimalPart.Len() < DecimalCount)
		{
			DecimalPart.InsertAt(DecimalPart.Len() - 1, FString::ChrN(DecimalCount - DecimalPart.Len(), TEXT('0')));
		}

		if (DecimalPart.Len() > DecimalCount)
		{
			DecimalPart.RemoveAt(DecimalCount, DecimalPart.Len());
		}

		if (!DecimalCharacter.IsEmpty())
		{
			DecimalPart.InsertAt(0, DecimalCharacter[0]);
		}
	}

	if (!PaddingCharacter.IsEmpty() && IntegerPart.Len() < IntegerCount)
	{
		IntegerPart.InsertAt(0, FString::ChrN(IntegerCount - IntegerPart.Len(), PaddingCharacter[0]));
	}

	if (!GroupingCharacter.IsEmpty() && GroupingSize > 0)
	{
		int32 ThousandsIndex = IntegerPart.Len() - 1;
		int32 ThousandsCount = 0;
		while (ThousandsIndex > 0)
		{
			if (IntegerPart[ThousandsIndex] != TEXT('.'))
			{
				ThousandsCount++;

				if (FMath::Modulo(ThousandsCount, GroupingSize) == 0 && ThousandsIndex != 0)
				{
					IntegerPart.InsertAt(ThousandsIndex, GroupingCharacter[0]);
				}
			}

			ThousandsIndex--;
		}
	}

	FString NumberString = IntegerPart + DecimalPart;

	if (bUseSign)
	{
		NumberString.InsertAt(0, bPositive ? TEXT('+') : TEXT('-'));
	}

	return NumberString;
}

UPropertyAnimatorCounter::UPropertyAnimatorCounter()
{
	if (!IsTemplate())
	{
		const TArray<FName> AvailableNames = GetAvailableFormatNames();

		if (!AvailableNames.IsEmpty())
		{
			PresetFormatName = AvailableNames[0];
		}
	}
}

void UPropertyAnimatorCounter::SetDisplayPattern(const FText& InPattern)
{
	DisplayPattern = InPattern;
}

void UPropertyAnimatorCounter::SetUseCustomFormat(bool bInUseCustom)
{
	if (bUseCustomFormat == bInUseCustom)
	{
		return;
	}

	bUseCustomFormat = bInUseCustom;
	OnUseCustomFormatChanged();
}

void UPropertyAnimatorCounter::SetPresetFormatName(FName InPresetName)
{
	if (InPresetName.IsEqual(PresetFormatName))
	{
		return;
	}

	if (!GetAvailableFormatNames().Contains(InPresetName))
	{
		return;
	}

	PresetFormatName = InPresetName;
}

void UPropertyAnimatorCounter::SetCustomFormat(const FPropertyAnimatorCounterFormat* InFormat)
{
	if (InFormat)
	{
		CustomFormat = TInstancedStruct<FPropertyAnimatorCounterFormat>::Make(*InFormat);
	}
	else
	{
		CustomFormat.Reset();
	}

	OnCustomFormatChanged();
}

const FPropertyAnimatorCounterFormat* UPropertyAnimatorCounter::GetCustomFormat() const
{
	return CustomFormat.GetPtr<FPropertyAnimatorCounterFormat>();
}

FString UPropertyAnimatorCounter::FormatNumber(double InNumber) const
{
	FText Output = FText::GetEmpty();

	if (const FPropertyAnimatorCounterFormat* Format = GetFormat())
	{
		Output = FText::Format(DisplayPattern, FText::FromString(Format->FormatNumber(InNumber)));
	}

	return Output.ToString();
}

const FPropertyAnimatorCounterFormat* UPropertyAnimatorCounter::GetFormat() const
{
	if (bUseCustomFormat)
	{
		return CustomFormat.GetPtr<FPropertyAnimatorCounterFormat>();
	}

	if (const UPropertyAnimatorSettings* AnimatorSettings = GetDefault<UPropertyAnimatorSettings>())
	{
		return AnimatorSettings->GetCounterFormat(PresetFormatName);
	}

	return nullptr;
}

#if WITH_EDITOR
FName UPropertyAnimatorCounter::GetUseCustomFormatPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCounter, bUseCustomFormat);
}

void UPropertyAnimatorCounter::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	// PECP for instanced struct does not provide the correct MemberProperty
	if (InPropertyChangedEvent.Property
		&& InPropertyChangedEvent.Property->GetOwnerStruct() == FPropertyAnimatorCounterFormat::StaticStruct())
	{
		OnCustomFormatChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCounter, bUseCustomFormat))
	{
		OnUseCustomFormatChanged();
	}
}

void UPropertyAnimatorCounter::OpenPropertyAnimatorSettings()
{
	if (const UPropertyAnimatorSettings* AnimatorSettings = GetDefault<UPropertyAnimatorSettings>())
	{
		AnimatorSettings->OpenSettings();
	}
}

void UPropertyAnimatorCounter::SaveCustomFormatAsPreset()
{
	if (const FPropertyAnimatorCounterFormat* Format = CustomFormat.GetPtr<FPropertyAnimatorCounterFormat>())
	{
		if (UPropertyAnimatorSettings* AnimatorSettings = GetMutableDefault<UPropertyAnimatorSettings>())
		{
			if (AnimatorSettings->AddCounterFormat(*Format))
			{
				PresetFormatName = Format->FormatName;
				bUseCustomFormat = false;
				CustomFormat.Reset();
			}
		}
	}
}
#endif

void UPropertyAnimatorCounter::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Counter");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Counter");
}

void UPropertyAnimatorCounter::EvaluateProperties(FInstancedPropertyBag& InParameters)
{
	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();

	EvaluateEachLinkedProperty([this, TimeElapsed](
		UPropertyAnimatorCoreContext* InContext
		, const FPropertyAnimatorCoreData& InResolvedProperty
		, FInstancedPropertyBag& InEvaluatedValues
		, int32 InRangeIndex
		, int32 InRangeMax)->bool
	{
		const FName PropertyHash = InResolvedProperty.GetLocatorPathHash();

		InEvaluatedValues.AddProperty(PropertyHash, EPropertyBagPropertyType::String);
		InEvaluatedValues.SetValueString(PropertyHash, FormatNumber(TimeElapsed));

		return true;
	});
}

bool UPropertyAnimatorCounter::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InValue->AsMutableObject();

		FString DisplayPatternValue = DisplayPattern.ToString();
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCounter, DisplayPattern), DisplayPatternValue);
		SetDisplayPattern(FText::FromString(DisplayPatternValue));

		FString PresetFormatNameValue = PresetFormatName.ToString();
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCounter, PresetFormatName), PresetFormatNameValue);
		FName NewPresetFormatName(PresetFormatNameValue);
		SetPresetFormatName(NewPresetFormatName);

		if (!PresetFormatName.IsEqual(NewPresetFormatName))
		{
			FPropertyAnimatorCounterFormat Format;
			Format.FormatName = NewPresetFormatName;

			uint64 IntegerCount = Format.MinIntegerCount;
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, MinIntegerCount), IntegerCount);
			Format.MinIntegerCount = IntegerCount;

			uint64 DecimalCount = Format.MaxDecimalCount;
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, MaxDecimalCount), DecimalCount);
			Format.MaxDecimalCount = DecimalCount;

			uint64 GroupingSize = Format.GroupingSize;
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, GroupingSize), GroupingSize);
			Format.GroupingSize = GroupingSize;

			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, DecimalCharacter), Format.DecimalCharacter);
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, GroupingCharacter), Format.GroupingCharacter);
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, PaddingCharacter), Format.PaddingCharacter);

			uint64 RoundingModeValue = static_cast<uint64>(Format.RoundingMode);
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, RoundingMode), RoundingModeValue);
			Format.RoundingMode = static_cast<EPropertyAnimatorCounterRoundingMode>(RoundingModeValue);

			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, bUseSign), Format.bUseSign);
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, bTruncate), Format.bTruncate);

			SetUseCustomFormat(true);
			SetCustomFormat(&Format);
		}

		return true;
	}

	return false;
}

bool UPropertyAnimatorCounter::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = OutValue->AsMutableObject();

		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCounter, DisplayPattern), DisplayPattern.ToString());

		if (const FPropertyAnimatorCounterFormat* Format = GetFormat())
		{
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCounter, PresetFormatName), bUseCustomFormat ? Format->FormatName.ToString() : PresetFormatName.ToString());
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, MinIntegerCount), static_cast<uint64>(Format->MinIntegerCount));
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, MaxDecimalCount), static_cast<uint64>(Format->MaxDecimalCount));
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, GroupingSize), static_cast<uint64>(Format->GroupingSize));
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, DecimalCharacter), Format->DecimalCharacter);
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, GroupingCharacter), Format->GroupingCharacter);
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, PaddingCharacter), Format->PaddingCharacter);
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, RoundingMode), static_cast<uint64>(Format->RoundingMode));
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, bUseSign), Format->bUseSign);
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCounterFormat, bTruncate), Format->bTruncate);
		}

		return true;
	}

	return false;
}

TArray<FName> UPropertyAnimatorCounter::GetAvailableFormatNames() const
{
	TArray<FName> FormatNames;

	if (const UPropertyAnimatorSettings* AnimatorSettings = GetDefault<UPropertyAnimatorSettings>())
	{
		FormatNames = AnimatorSettings->GetCounterFormatNames().Array();
	}

	return FormatNames;
}

void UPropertyAnimatorCounter::OnCustomFormatChanged()
{
	if (FPropertyAnimatorCounterFormat* Format = CustomFormat.GetMutablePtr<FPropertyAnimatorCounterFormat>())
	{
		Format->EnsureCharactersLength();
	}
}

void UPropertyAnimatorCounter::OnUseCustomFormatChanged()
{
	// Start from the current selected preset format
	if (bUseCustomFormat && !CustomFormat.IsValid())
	{
		if (const UPropertyAnimatorSettings* AnimatorSettings = GetDefault<UPropertyAnimatorSettings>())
		{
			CustomFormat = TInstancedStruct<FPropertyAnimatorCounterFormat>::Make(*AnimatorSettings->GetCounterFormat(PresetFormatName));
		}
	}
}

#undef LOCTEXT_NAMESPACE
