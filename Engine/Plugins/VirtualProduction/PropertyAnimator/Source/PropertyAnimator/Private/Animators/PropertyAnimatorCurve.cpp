// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorCurve.h"

#include "Curves/PropertyAnimatorEaseCurve.h"
#include "Curves/PropertyAnimatorWaveCurve.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "UObject/ConstructorHelpers.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorCurve"

UPropertyAnimatorCurve::UPropertyAnimatorCurve()
{
	static const ConstructorHelpers::FObjectFinder<UPropertyAnimatorWaveCurve> BaseCurve(TEXT("/Script/PropertyAnimator.PropertyAnimatorWaveCurve'/PropertyAnimator/Waves/Constant.Constant'"));

	if (BaseCurve.Succeeded())
	{
		WaveCurve = BaseCurve.Object;
	}

	static const ConstructorHelpers::FObjectFinder<UPropertyAnimatorEaseCurve> LinearCurve(TEXT("/Script/PropertyAnimator.PropertyAnimatorEaseCurve'/PropertyAnimator/Eases/Linear.Linear'"));

	if (LinearCurve.Succeeded())
	{
		EaseIn.EaseCurve = LinearCurve.Object;
		EaseOut.EaseCurve = LinearCurve.Object;
	}
}

void UPropertyAnimatorCurve::SetWaveCurve(UPropertyAnimatorWaveCurve* InCurve)
{
	WaveCurve = InCurve;
}

void UPropertyAnimatorCurve::SetEaseInEnabled(bool bInEnabled)
{
	bEaseInEnabled = bInEnabled;
}

void UPropertyAnimatorCurve::SetEaseIn(const FPropertyAnimatorCurveEasing& InEasing)
{
	EaseIn = InEasing;
}

void UPropertyAnimatorCurve::SetEaseOutEnabled(bool bInEnabled)
{
	bEaseOutEnabled = bInEnabled;
}

void UPropertyAnimatorCurve::SetEaseOut(const FPropertyAnimatorCurveEasing& InEasing)
{
	EaseOut = InEasing;
}

#if WITH_EDITOR
void UPropertyAnimatorCurve::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCurve, EaseIn))
	{
		OnEaseInChanged();
	}
	else if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorCurve, EaseOut))
	{
		OnEaseOutChanged();
	}
}
#endif

void UPropertyAnimatorCurve::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Curve");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Curve");
}

bool UPropertyAnimatorCurve::EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
{
	if (!WaveCurve)
	{
		return false;
	}

	const FRichCurve& SampleCurve = WaveCurve->FloatCurve;

	float MinTime = 0;
	float MaxTime = 0;
	SampleCurve.GetTimeRange(MinTime, MaxTime);

	float MinValue = 0;
	float MaxValue = 0;
	SampleCurve.GetValueRange(MinValue, MaxValue);

	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();
	const double Frequency = InParameters.GetValueDouble(FrequencyParameterName).GetValue();
	const double Period = 1.0 / Frequency;

	const double SampleTime = FMath::Wrap(TimeElapsed, 0.0, Period);
	const float NormalizedSampleTime = FMath::GetMappedRangeValueClamped(FVector2D(0, Period), FVector2D(MinTime, MaxTime), SampleTime);

	const float SampleValue = SampleCurve.Eval(NormalizedSampleTime);
	float SampleValueNormalized = FMath::GetMappedRangeValueClamped(FVector2D(MinValue, MaxValue), FVector2D(0, 1), SampleValue);

	if (bEaseInEnabled && EaseIn.EaseCurve && SampleTime < EaseIn.EaseDuration)
	{
		const FRichCurve& EaseCurve = EaseIn.EaseCurve->FloatCurve;
		const float EaseTimeNormalized = FMath::GetMappedRangeValueClamped(FVector2D(0, EaseIn.EaseDuration), FVector2D(0, 1), SampleTime);
		SampleValueNormalized *= EaseCurve.Eval(EaseTimeNormalized);
	}

	if (bEaseOutEnabled && EaseOut.EaseCurve && SampleTime > (CycleDuration - EaseOut.EaseDuration))
	{
		const FRichCurve& EaseCurve = EaseOut.EaseCurve->FloatCurve;
		const float EaseTimeNormalized = 1 - FMath::GetMappedRangeValueClamped(FVector2D(CycleDuration - EaseOut.EaseDuration, CycleDuration), FVector2D(0, 1), SampleTime);
		SampleValueNormalized *= EaseCurve.Eval(EaseTimeNormalized);
	}

	InParameters.AddProperty(AlphaParameterName, EPropertyBagPropertyType::Float);
	InParameters.SetValueFloat(AlphaParameterName, SampleValueNormalized);

	return InContext->EvaluateProperty(InPropertyData, InParameters, OutEvaluationResult);
}

bool UPropertyAnimatorCurve::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InValue->AsMutableObject();

		FString WaveCurveValue;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, WaveCurve), WaveCurveValue);

		if (UPropertyAnimatorWaveCurve* Curve = LoadObject<UPropertyAnimatorWaveCurve>(nullptr, *WaveCurveValue))
		{
			SetWaveCurve(Curve);
		}

		bool bEaseInEnabledValue = bEaseInEnabled;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, bEaseInEnabled), bEaseInEnabledValue);
		SetEaseInEnabled(bEaseInEnabledValue);

		bool bEaseOutEnabledValue = bEaseOutEnabled;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, bEaseOutEnabled), bEaseOutEnabledValue);
		SetEaseOutEnabled(bEaseOutEnabledValue);

		TSharedPtr<FPropertyAnimatorCorePresetArchive> EaseInArchive;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, EaseIn), EaseInArchive);

		if (const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> EaseInObject = EaseInArchive->AsMutableObject())
		{
			FString EaseCurvePath;
			EaseInObject->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseCurve), EaseCurvePath);

			double EaseDuration;
			EaseInObject->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseDuration), EaseDuration);

			if (UPropertyAnimatorEaseCurve* EaseCurve = LoadObject<UPropertyAnimatorEaseCurve>(nullptr, *EaseCurvePath))
			{
				FPropertyAnimatorCurveEasing Easing;
				Easing.EaseCurve = EaseCurve;
				Easing.EaseDuration = EaseDuration;
				SetEaseIn(Easing);
			}
		}

		TSharedPtr<FPropertyAnimatorCorePresetArchive> EaseOutArchive;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, EaseOut), EaseOutArchive);

		if (const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> EaseOutObject = EaseOutArchive->AsMutableObject())
		{
			FString EaseCurvePath;
			EaseOutObject->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseCurve), EaseCurvePath);

			double EaseDuration;
			EaseOutObject->Get(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseDuration), EaseDuration);

			if (UPropertyAnimatorEaseCurve* EaseCurve = LoadObject<UPropertyAnimatorEaseCurve>(nullptr, *EaseCurvePath))
			{
				FPropertyAnimatorCurveEasing Easing;
				Easing.EaseCurve = EaseCurve;
				Easing.EaseDuration = EaseDuration;
				SetEaseOut(Easing);
			}
		}

		return true;
	}

	return false;
}

bool UPropertyAnimatorCurve::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = OutValue->AsMutableObject();

		if (WaveCurve)
		{
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, WaveCurve), WaveCurve.GetPath());
		}

		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, bEaseInEnabled), bEaseInEnabled);
		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, bEaseOutEnabled), bEaseOutEnabled);

		if (EaseIn.EaseCurve)
		{
			TSharedRef<FPropertyAnimatorCorePresetObjectArchive> EaseObject = InPreset->GetArchiveImplementation()->CreateObject();
			EaseObject->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseCurve), EaseIn.EaseCurve.GetPath());
			EaseObject->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseDuration), EaseIn.EaseDuration);
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, EaseIn), EaseObject);
		}

		if (EaseOut.EaseCurve)
		{
			TSharedRef<FPropertyAnimatorCorePresetObjectArchive> EaseObject = InPreset->GetArchiveImplementation()->CreateObject();
			EaseObject->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseCurve), EaseOut.EaseCurve.GetPath());
			EaseObject->Set(GET_MEMBER_NAME_STRING_CHECKED(FPropertyAnimatorCurveEasing, EaseDuration), EaseOut.EaseDuration);
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorCurve, EaseOut), EaseObject);
		}

		return true;
	}

	return false;
}

void UPropertyAnimatorCurve::OnEaseInChanged()
{
	EaseIn.EaseDuration = FMath::Clamp(EaseIn.EaseDuration, 0, CycleDuration - EaseOut.EaseDuration);
}

void UPropertyAnimatorCurve::OnEaseOutChanged()
{
	EaseOut.EaseDuration = FMath::Clamp(EaseOut.EaseDuration, 0, CycleDuration - EaseIn.EaseDuration);
}

void UPropertyAnimatorCurve::OnCycleDurationChanged()
{
	Super::OnCycleDurationChanged();

	OnEaseInChanged();
	OnEaseOutChanged();
}

#undef LOCTEXT_NAMESPACE
