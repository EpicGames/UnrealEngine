// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorNumericBase.h"

#include "Properties/PropertyAnimatorFloatContext.h"
#include "Properties/PropertyAnimatorRotatorContext.h"
#include "Properties/PropertyAnimatorVectorContext.h"
#include "Properties/Converters/PropertyAnimatorCoreConverterBase.h"
#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"
#include "TimeSources/PropertyAnimatorCoreSystemTimeSource.h"
#include "TimeSources/PropertyAnimatorCoreTimeSourceBase.h"

#if WITH_EDITOR
void UPropertyAnimatorNumericBase::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	static const FName RandomTimeOffsetName = GET_MEMBER_NAME_CHECKED(UPropertyAnimatorNumericBase, bRandomTimeOffset);
	static const FName SeedName = GET_MEMBER_NAME_CHECKED(UPropertyAnimatorNumericBase, Seed);

	if (MemberName == SeedName
		|| MemberName == RandomTimeOffsetName)
	{
		OnSeedChanged();
	}
}
#endif

void UPropertyAnimatorNumericBase::SetMagnitude(float InMagnitude)
{
	if (FMath::IsNearlyEqual(Magnitude, InMagnitude))
	{
		return;
	}

	Magnitude = InMagnitude;
	OnMagnitudeChanged();
}

void UPropertyAnimatorNumericBase::SetCycleDuration(float InCycleDuration)
{
	if (FMath::IsNearlyEqual(CycleDuration, InCycleDuration))
	{
		return;
	}

	CycleDuration = InCycleDuration;
	OnCycleDurationChanged();
}

void UPropertyAnimatorNumericBase::SetCycleGapDuration(float InCycleGap)
{
	CycleGapDuration = FMath::Max(0, InCycleGap);
}

void UPropertyAnimatorNumericBase::SetCycleMode(EPropertyAnimatorCycleMode InMode)
{
	if (CycleMode == InMode)
	{
		return;
	}

	CycleMode = InMode;
	OnCycleModeChanged();
}

void UPropertyAnimatorNumericBase::SetRandomTimeOffset(bool bInOffset)
{
	if (bRandomTimeOffset == bInOffset)
	{
		return;
	}

	bRandomTimeOffset = bInOffset;
	OnSeedChanged();
}

void UPropertyAnimatorNumericBase::SetSeed(int32 InSeed)
{
	if (Seed == InSeed)
	{
		return;
	}

	Seed = InSeed;
	OnSeedChanged();
}

TSubclassOf<UPropertyAnimatorCoreContext> UPropertyAnimatorNumericBase::GetPropertyContextClass(const FPropertyAnimatorCoreData& InProperty)
{
	if (InProperty.IsA<FStructProperty>())
	{
		const FName TypeName = InProperty.GetLeafPropertyTypeName();

		if (TypeName == NAME_Rotator)
		{
			return UPropertyAnimatorRotatorContext::StaticClass();
		}

		if (TypeName == NAME_Vector)
		{
			return UPropertyAnimatorVectorContext::StaticClass();
		}
	}

	return UPropertyAnimatorFloatContext::StaticClass();
}

EPropertyAnimatorPropertySupport UPropertyAnimatorNumericBase::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	const FName TypeName = InPropertyData.GetLeafPropertyTypeName();

	if (InPropertyData.IsA<FFloatProperty>() || InPropertyData.IsA<FDoubleProperty>())
	{
		return EPropertyAnimatorPropertySupport::Complete;
	}

	if (InPropertyData.IsA<FStructProperty>())
	{
		if (TypeName == NAME_Rotator)
		{
			return EPropertyAnimatorPropertySupport::Complete;
		}

		if (TypeName == NAME_Vector)
		{
			return EPropertyAnimatorPropertySupport::Complete;
		}
	}

	// Check if a converter supports the conversion
	if (UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		static const FPropertyBagPropertyDesc AnimatorTypeDesc("", EPropertyBagPropertyType::Float);
		const FPropertyBagPropertyDesc PropertyTypeDesc("", InPropertyData.GetLeafProperty());

		if (AnimatorSubsystem->IsConversionSupported(AnimatorTypeDesc, PropertyTypeDesc))
		{
			return EPropertyAnimatorPropertySupport::Incomplete;
		}
	}

	return Super::IsPropertySupported(InPropertyData);
}

void UPropertyAnimatorNumericBase::EvaluateProperties(FInstancedPropertyBag& InParameters)
{
	const float AnimatorMagnitude = Magnitude * InParameters.GetValueFloat(MagnitudeParameterName).GetValue();
	double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();
	RandomStream = FRandomStream(Seed);

	EvaluateEachLinkedProperty([this, &TimeElapsed, &AnimatorMagnitude, &InParameters](
		UPropertyAnimatorCoreContext* InOptions
		, const FPropertyAnimatorCoreData& InResolvedProperty
		, FInstancedPropertyBag& InEvaluatedValues
		, int32 InRangeIndex
		, int32 InRangeMax)->bool
	{
		const double RandomTimeOffset = bRandomTimeOffset ? RandomStream.GetFraction() : 0;

		const int32 RangeIndex = InRangeIndex + 1;
		const int32 RangeMax = InRangeMax + 1;
		const double TimeOffset = InOptions->GetTimeOffset() / RangeMax;
		double PropertyTimeElapsed = TimeElapsed + RandomTimeOffset;
		double Frequency = CycleDuration != 0 ? 1.0 / static_cast<double>(CycleDuration) : 0.0;

		// handle time offset for range
		if (TimeOffset >= 0.0)
		{
		    // positive offset: last element starts first
		    PropertyTimeElapsed += RangeIndex * TimeOffset;
		}
		else
		{
		    // negative offset: first element starts first
		    PropertyTimeElapsed += (InRangeMax - InRangeIndex) * FMath::Abs(TimeOffset);
		}

		// wrap for do once
		if (CycleMode == EPropertyAnimatorCycleMode::DoOnce)
		{
		    if (PropertyTimeElapsed < 0.0)
		    {
		        PropertyTimeElapsed = 0.0;
		    }
		    else if (PropertyTimeElapsed > CycleDuration)
		    {
		        PropertyTimeElapsed = CycleDuration - UE_KINDA_SMALL_NUMBER;
		    }
		}
		else
		{
			// wrap negative time
		    if (PropertyTimeElapsed < 0.0)
		    {
		        double Period = CycleDuration + CycleGapDuration;
		        if (CycleMode == EPropertyAnimatorCycleMode::PingPong)
		        {
		            Period = 2.0 * CycleDuration + 2.0 * CycleGapDuration;
		        }

		        PropertyTimeElapsed = FMath::Wrap(PropertyTimeElapsed, 0.0, Period);
		        if (PropertyTimeElapsed < 0.0)
		        {
		            PropertyTimeElapsed += Period;
		        }
		    }

		    if (CycleMode == EPropertyAnimatorCycleMode::Loop)
		    {
		        const double Period = CycleDuration + CycleGapDuration - UE_KINDA_SMALL_NUMBER;
		        const double TimeInPeriod = FMath::Wrap(PropertyTimeElapsed, 0.0, Period);

		        if (TimeInPeriod <= CycleDuration)
		        {
		        	// active phase
		            PropertyTimeElapsed = TimeInPeriod;
		        }
		        else
		        {
		        	// gap phase
		            PropertyTimeElapsed = CycleDuration - UE_KINDA_SMALL_NUMBER;
		        }
		    }
		    else if (CycleMode == EPropertyAnimatorCycleMode::PingPong)
		    {
		        const double Period = 2.0 * CycleDuration + 2.0 * CycleGapDuration - UE_KINDA_SMALL_NUMBER;
		        const double TimeInPeriod = FMath::Wrap(PropertyTimeElapsed, 0.0, Period);

		        if (TimeInPeriod < CycleDuration)
		        {
		        	// forward phase
		            PropertyTimeElapsed = TimeInPeriod;
		        }
		        else if (TimeInPeriod < CycleDuration + CycleGapDuration)
		        {
		        	// forward gap hold
		            PropertyTimeElapsed = CycleDuration - UE_KINDA_SMALL_NUMBER;
		        }
		        else if (TimeInPeriod < 2.0 * CycleDuration + CycleGapDuration)
		        {
		        	// backward phase
		            PropertyTimeElapsed = 2.0 * CycleDuration + CycleGapDuration - TimeInPeriod;
		        }
		        else
		        {
		        	// backward gap hold
		            PropertyTimeElapsed = 0.0;
		        }
		    }
		    else if (CycleMode == EPropertyAnimatorCycleMode::None)
		    {
		        Frequency = 1;
		    }
		}

		if (Magnitude != 0
			&& CycleDuration > 0
			&& InOptions->GetMagnitude() != 0)
		{
			// Frequency
			InParameters.AddProperty(FrequencyParameterName, EPropertyBagPropertyType::Float);
			InParameters.SetValueFloat(FrequencyParameterName, Frequency);

			// Time Elapsed
			InParameters.SetValueDouble(TimeElapsedParameterName, PropertyTimeElapsed);

			// Magnitude
			InParameters.SetValueFloat(MagnitudeParameterName, AnimatorMagnitude * InOptions->GetMagnitude());

			return EvaluateProperty(InResolvedProperty, InOptions, InParameters, InEvaluatedValues);
		}

		return false;
	});
}

void UPropertyAnimatorNumericBase::OnPropertyLinked(UPropertyAnimatorCoreContext* InLinkedProperty, EPropertyAnimatorPropertySupport InSupport)
{
	Super::OnPropertyLinked(InLinkedProperty, InSupport);

	if (EnumHasAnyFlags(InSupport, EPropertyAnimatorPropertySupport::Incomplete))
	{
		if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
		{
			static const FPropertyBagPropertyDesc AnimatorTypeDesc("", EPropertyBagPropertyType::Float);
			const FPropertyBagPropertyDesc PropertyTypeDesc("", InLinkedProperty->GetAnimatedProperty().GetLeafProperty());

			const TSet<UPropertyAnimatorCoreConverterBase*> Converters = AnimatorSubsystem->GetSupportedConverters(AnimatorTypeDesc, PropertyTypeDesc);

			check(!Converters.IsEmpty())

			InLinkedProperty->SetConverterClass(Converters.Array()[0]->GetClass());
		}
	}
}

bool UPropertyAnimatorNumericBase::IsTimeSourceSupported(UPropertyAnimatorCoreTimeSourceBase* InTimeSource) const
{
	return !InTimeSource->IsA<UPropertyAnimatorCoreSystemTimeSource>();
}

void UPropertyAnimatorNumericBase::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Category = TEXT("Numeric");
}

bool UPropertyAnimatorNumericBase::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InValue->AsMutableObject();

		double MagnitudeValue = Magnitude;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, Magnitude), MagnitudeValue);
		SetMagnitude(MagnitudeValue);

		if (CycleMode != EPropertyAnimatorCycleMode::None)
		{
			uint64 CycleModeValue = static_cast<uint64>(CycleMode);
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, CycleMode), CycleModeValue);
			SetCycleMode(static_cast<EPropertyAnimatorCycleMode>(CycleModeValue));

			double CycleDurationValue = CycleDuration;
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, CycleDuration), CycleDurationValue);
			SetCycleDuration(CycleDurationValue);

			double CycleGapDurationValue = CycleGapDuration;
			AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, CycleGapDuration), CycleGapDurationValue);
			SetCycleGapDuration(CycleGapDurationValue);
		}

		bool bRandomTimeOffsetValue = bRandomTimeOffset;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, bRandomTimeOffset), bRandomTimeOffsetValue);
		SetRandomTimeOffset(bRandomTimeOffsetValue);

		int64 SeedValue = Seed;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, Seed), SeedValue);
		SetSeed(SeedValue);

		return true;
	}

	return false;
}

bool UPropertyAnimatorNumericBase::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = OutValue->AsMutableObject();

		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, Magnitude), Magnitude);

		if (CycleMode != EPropertyAnimatorCycleMode::None)
		{
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, CycleMode), static_cast<uint64>(CycleMode));
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, CycleDuration), CycleDuration);
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, CycleGapDuration), CycleGapDuration);
		}

		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, bRandomTimeOffset), bRandomTimeOffset);
		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorNumericBase, Seed), static_cast<int64>(Seed));

		return true;
	}

	return false;
}
