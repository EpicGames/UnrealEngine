// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorOscillate.h"

#include "Properties/PropertyAnimatorFloatContext.h"
#include "PropertyAnimatorShared.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorOscillate"

void UPropertyAnimatorOscillate::SetOscillateFunction(EPropertyAnimatorOscillateFunction InFunction)
{
	OscillateFunction = InFunction;
}

void UPropertyAnimatorOscillate::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Oscillate");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Oscillate");
}

bool UPropertyAnimatorOscillate::EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
{
	double WaveResult = 0.f;

	using namespace UE::PropertyAnimator;

	const float Frequency = InParameters.GetValueDouble(FrequencyParameterName).GetValue();
	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();

	constexpr double Amplitude = 1;
	constexpr double Offset = 0;

	switch (OscillateFunction)
	{
		case EPropertyAnimatorOscillateFunction::Sine:
			WaveResult = Wave::Sine(TimeElapsed, Amplitude, Frequency, Offset);
			break;
		case EPropertyAnimatorOscillateFunction::Cosine:
			WaveResult = Wave::Cosine(TimeElapsed, Amplitude, Frequency, Offset);
			break;
		case EPropertyAnimatorOscillateFunction::Square:
			WaveResult = Wave::Square(TimeElapsed, Amplitude, Frequency, Offset);
			break;
		case EPropertyAnimatorOscillateFunction::InvertedSquare:
			WaveResult = Wave::InvertedSquare(TimeElapsed, Amplitude, Frequency, Offset);
			break;
		case EPropertyAnimatorOscillateFunction::Sawtooth:
			WaveResult = Wave::Sawtooth(TimeElapsed, Amplitude, Frequency, Offset);
			break;
		case EPropertyAnimatorOscillateFunction::Triangle:
			WaveResult = Wave::Triangle(TimeElapsed, Amplitude, Frequency, Offset);
			break;
		default:
			checkNoEntry();
	}

	const double NormalizedResult = FMath::GetMappedRangeValueClamped(FVector2D(-1, 1), FVector2D(0, 1), WaveResult);

	InParameters.AddProperty(AlphaParameterName, EPropertyBagPropertyType::Float);
	InParameters.SetValueFloat(AlphaParameterName, NormalizedResult);

	return InContext->EvaluateProperty(InPropertyData, InParameters, OutEvaluationResult);
}

#undef LOCTEXT_NAMESPACE
