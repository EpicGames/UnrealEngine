// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorPulse.h"

#include "Properties/PropertyAnimatorFloatContext.h"
#include "PropertyAnimatorShared.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPulse"

void UPropertyAnimatorPulse::SetEasingFunction(EPropertyAnimatorEasingFunction InEasingFunction)
{
	EasingFunction = InEasingFunction;
}

void UPropertyAnimatorPulse::SetEasingType(EPropertyAnimatorEasingType InEasingType)
{
	EasingType = InEasingType;
}

void UPropertyAnimatorPulse::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Pulse");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Pulse");
}

bool UPropertyAnimatorPulse::EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
{
	const float Frequency = InParameters.GetValueDouble(FrequencyParameterName).GetValue();
	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();

	constexpr double Amplitude = 1;
	constexpr double Offset = 0;

	const double WaveResult = UE::PropertyAnimator::Wave::Triangle(TimeElapsed, Amplitude, Frequency, Offset);

	// Result of wave functions is [-1, 1] -> remap to [0, 1] range for easing functions
	const float NormalizedWaveProgress = FMath::GetMappedRangeValueClamped(FVector2D(-1, 1), FVector2D(0, 1), WaveResult);

	// Apply easing function on normalized progress
	const float EasingResult = UE::PropertyAnimator::Easing::Ease(NormalizedWaveProgress, EasingFunction, EasingType);

	InParameters.AddProperty(AlphaParameterName, EPropertyBagPropertyType::Float);
	InParameters.SetValueFloat(AlphaParameterName, EasingResult);

	return InContext->EvaluateProperty(InPropertyData, InParameters, OutEvaluationResult);
}

#undef LOCTEXT_NAMESPACE