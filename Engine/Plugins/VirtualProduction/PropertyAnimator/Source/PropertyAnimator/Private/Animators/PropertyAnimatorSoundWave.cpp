// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorSoundWave.h"

#include "LoudnessNRT.h"
#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "Sound/SoundWave.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorSoundWave"

UPropertyAnimatorSoundWave::UPropertyAnimatorSoundWave()
{
	CycleMode = EPropertyAnimatorCycleMode::None;
}

void UPropertyAnimatorSoundWave::SetSampledSoundWave(USoundWave* InSoundWave)
{
	if (SampledSoundWave == InSoundWave)
	{
		return;
	}

	SampledSoundWave = InSoundWave;
	OnSampledSoundWaveChanged();
}

void UPropertyAnimatorSoundWave::SetLoop(bool bInLoop)
{
	bLoop = bInLoop;
}

#if WITH_EDITOR
void UPropertyAnimatorSoundWave::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName MemberName = PropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorSoundWave, SampledSoundWave))
	{
		OnSampledSoundWaveChanged();
	}
}
#endif // WITH_EDITOR

void UPropertyAnimatorSoundWave::OnSampledSoundWaveChanged()
{
	if (!AudioAnalyzer)
	{
		AudioAnalyzer = NewObject<ULoudnessNRT>(this);
	}

	AudioAnalyzer->Sound = SampledSoundWave;

#if WITH_EDITOR
	// Needed to analyse new audio sample
	FProperty* SoundProperty = FindFProperty<FProperty>(ULoudnessNRT::StaticClass(), GET_MEMBER_NAME_CHECKED(ULoudnessNRT, Sound));
	FPropertyChangedEvent PropertyChangedEvent(SoundProperty, EPropertyChangeType::ValueSet);
	AudioAnalyzer->PostEditChangeProperty(PropertyChangedEvent);
#endif
}

void UPropertyAnimatorSoundWave::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("SoundWave");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "SoundWave");
}

bool UPropertyAnimatorSoundWave::EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
{
	if (AudioAnalyzer && AudioAnalyzer->DurationInSeconds > 0)
	{
		double SampleTime = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();

		if ((SampleTime >= 0 && SampleTime <= AudioAnalyzer->DurationInSeconds) || bLoop)
		{
			SampleTime = FMath::Wrap<double>(SampleTime, 0.0, AudioAnalyzer->DurationInSeconds);

			if (SampleTime < 0)
			{
				SampleTime = AudioAnalyzer->DurationInSeconds + SampleTime;
			}

			float NormalizedLoudness = 0.f;
			AudioAnalyzer->GetNormalizedLoudnessAtTime(SampleTime, NormalizedLoudness);

			InParameters.AddProperty(AlphaParameterName, EPropertyBagPropertyType::Float);
			InParameters.SetValueFloat(AlphaParameterName, NormalizedLoudness);

			return InContext->EvaluateProperty(InPropertyData, InParameters, OutEvaluationResult);
		}
	}

	return false;
}

bool UPropertyAnimatorSoundWave::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InValue->AsMutableObject();

		FString SoundWaveValue;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorSoundWave, SampledSoundWave), SoundWaveValue);

		if (USoundWave* SoundWave = LoadObject<USoundWave>(nullptr, *SoundWaveValue))
		{
			SetSampledSoundWave(SoundWave);
		}

		bool bLoopValue = bLoop;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorSoundWave, bLoop), bLoop);
		SetLoop(bLoopValue);

		return true;
	}

	return false;
}

bool UPropertyAnimatorSoundWave::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = OutValue->AsMutableObject();

		if (SampledSoundWave)
		{
			AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorSoundWave, SampledSoundWave), SampledSoundWave.GetPath());
		}

		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorSoundWave, bLoop), bLoop);

		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE