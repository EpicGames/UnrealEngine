// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorTextBase.h"

#include "Properties/Converters/PropertyAnimatorCoreConverterBase.h"
#include "Subsystems/PropertyAnimatorCoreSubsystem.h"

EPropertyAnimatorPropertySupport UPropertyAnimatorTextBase::IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const
{
	if (InPropertyData.IsA<FStrProperty>())
	{
		return EPropertyAnimatorPropertySupport::Complete;
	}

	// Check if a converter supports the conversion
	if (UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
	{
		static const FPropertyBagPropertyDesc AnimatorTypeDesc("", EPropertyBagPropertyType::String);
		const FPropertyBagPropertyDesc PropertyTypeDesc("", InPropertyData.GetLeafProperty());

		if (AnimatorSubsystem->IsConversionSupported(AnimatorTypeDesc, PropertyTypeDesc))
		{
			return EPropertyAnimatorPropertySupport::Incomplete;
		}
	}

	return Super::IsPropertySupported(InPropertyData);
}

void UPropertyAnimatorTextBase::OnPropertyLinked(UPropertyAnimatorCoreContext* InLinkedProperty, EPropertyAnimatorPropertySupport InSupport)
{
	Super::OnPropertyLinked(InLinkedProperty, InSupport);

	if (EnumHasAnyFlags(InSupport, EPropertyAnimatorPropertySupport::Incomplete))
	{
		if (const UPropertyAnimatorCoreSubsystem* AnimatorSubsystem = UPropertyAnimatorCoreSubsystem::Get())
		{
			static const FPropertyBagPropertyDesc AnimatorTypeDesc("", EPropertyBagPropertyType::String);

			const FPropertyAnimatorCoreData& Property = InLinkedProperty->GetAnimatedProperty();
			const FPropertyBagPropertyDesc PropertyTypeDesc("", Property.GetLeafProperty());
			const TSet<UPropertyAnimatorCoreConverterBase*> Converters = AnimatorSubsystem->GetSupportedConverters(AnimatorTypeDesc, PropertyTypeDesc);
			check(!Converters.IsEmpty())

			InLinkedProperty->SetConverterClass(Converters.Array()[0]->GetClass());
		}
	}
}

void UPropertyAnimatorTextBase::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Category = TEXT("Text");
}
