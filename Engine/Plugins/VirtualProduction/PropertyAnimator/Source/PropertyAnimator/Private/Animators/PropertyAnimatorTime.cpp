// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorTime.h"

#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Properties/PropertyAnimatorFloatContext.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorTime"

void UPropertyAnimatorTime::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Time");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Time");
}

bool UPropertyAnimatorTime::EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
{
	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();
	const double Frequency = InParameters.GetValueDouble(FrequencyParameterName).GetValue();
	const double TimePeriod = 1.f / Frequency;
	const double TimeProgress = FMath::Wrap(TimeElapsed, 0.0, TimePeriod);
	const float NormalizedValue = FMath::GetMappedRangeValueClamped(FVector2f(0, TimePeriod), FVector2f(0, 1), TimeProgress);

	InParameters.AddProperty(AlphaParameterName, EPropertyBagPropertyType::Float);
	InParameters.SetValueFloat(AlphaParameterName, NormalizedValue);

	return InContext->EvaluateProperty(InPropertyData, InParameters, OutEvaluationResult);
}

#undef LOCTEXT_NAMESPACE
