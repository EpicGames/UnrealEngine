// Copyright Epic Games, Inc. All Rights Reserved.

#include "Animators/PropertyAnimatorWiggle.h"

#include "Properties/Handlers/PropertyAnimatorCoreHandlerBase.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "PropertyAnimatorShared.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorWiggle"

UPropertyAnimatorWiggle::UPropertyAnimatorWiggle()
{
	static int32 SeedIncrement = 0;

	bRandomTimeOffset = true;
	Seed = SeedIncrement++;
	CycleMode = EPropertyAnimatorCycleMode::None;
}

void UPropertyAnimatorWiggle::SetFrequency(float InFrequency)
{
	Frequency = FMath::Max(0, InFrequency);
}

void UPropertyAnimatorWiggle::OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata)
{
	Super::OnAnimatorRegistered(InMetadata);

	InMetadata.Name = TEXT("Wiggle");
	InMetadata.DisplayName = LOCTEXT("AnimatorDisplayName", "Wiggle");
}

bool UPropertyAnimatorWiggle::EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
{
	const double TimeElapsed = InParameters.GetValueDouble(TimeElapsedParameterName).GetValue();

	// Apply random wave based on time and frequency
	const double WaveResult = UE::PropertyAnimator::Wave::Perlin(TimeElapsed, 1.f, Frequency, 0.f);

	// Remap from [-1, 1] to user amplitude from [Min, Max]
	const float NormalizedValue = FMath::GetMappedRangeValueClamped(FVector2D(-1, 1), FVector2D(0, 1), WaveResult);

	InParameters.AddProperty(AlphaParameterName, EPropertyBagPropertyType::Float);
	InParameters.SetValueFloat(AlphaParameterName, NormalizedValue);

	return InContext->EvaluateProperty(InPropertyData, InParameters, OutEvaluationResult);
}

bool UPropertyAnimatorWiggle::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = InValue->AsMutableObject();

		double FrequencyValue = Frequency;
		AnimatorArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorWiggle, Frequency), FrequencyValue);
		SetFrequency(FrequencyValue);

		return true;
	}

	return false;
}

bool UPropertyAnimatorWiggle::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> AnimatorArchive = OutValue->AsMutableObject();

		AnimatorArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorWiggle, Frequency), Frequency);

		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
