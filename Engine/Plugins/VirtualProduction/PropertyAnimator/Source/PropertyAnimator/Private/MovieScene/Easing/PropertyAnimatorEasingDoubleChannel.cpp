// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Easing/PropertyAnimatorEasingDoubleChannel.h"
#include "MovieScene.h"
#include "MovieScene/PropertyAnimatorMovieSceneUtils.h"
#include "MovieSceneSection.h"
#include "PropertyAnimatorShared.h"

double FPropertyAnimatorEasingDoubleChannel::Evaluate(double InBaseSeconds, double InSeconds) const
{
	return Evaluate(Parameters, InBaseSeconds, InSeconds);
}

bool FPropertyAnimatorEasingDoubleChannel::Evaluate(const UMovieSceneSection* InSection, FFrameTime InTime, double& OutValue) const
{
	if (ensure(InSection))
	{
		if (const UMovieScene* MovieScene = InSection->GetTypedOuter<UMovieScene>())
		{
			FFrameRate TickResolution = MovieScene->GetTickResolution();
			FFrameTime BaseTime = FPropertyAnimatorMovieSceneUtils::GetBaseTime(*InSection, *MovieScene);
			OutValue = Evaluate(Parameters, TickResolution.AsSeconds(BaseTime), TickResolution.AsSeconds(InTime));
			return true;
		}
	}
	return false;
}

double FPropertyAnimatorEasingDoubleChannel::Evaluate(const FPropertyAnimatorEasingParameters& InParameters, double InBaseSeconds, double InSeconds)
{
	double Progress = 0.0;

	if (!FMath::IsNearlyZero(InParameters.EasingTime))
	{
		Progress = (InSeconds + InParameters.OffsetX - InBaseSeconds) / InParameters.EasingTime;
		Progress = FMath::Clamp(Progress, 0.0, 1.0);
	}

	// Reverse Progress since 0 represents Loc 0 (target location), and 1 represents max value
	Progress = 1.0 - Progress;

	return InParameters.OffsetY + (InParameters.Amplitude * UE::PropertyAnimator::Easing::Ease(Progress, InParameters.EasingFunction, InParameters.EasingType));
}
