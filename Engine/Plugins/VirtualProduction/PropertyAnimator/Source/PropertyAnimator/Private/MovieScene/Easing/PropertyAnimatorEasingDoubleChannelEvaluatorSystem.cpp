// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorEasingDoubleChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "MovieScene/PropertyAnimatorComponentTypes.h"
#include "MovieScene/Easing/PropertyAnimatorEasingDoubleChannel.h"
#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate Easing Double Channels"), MovieSceneEval_EvaluatePropertyAnimatorEasingDoubleChannelTask, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{
	struct FPropertyAnimatorEvaluateEasingDoubleChannels
	{
		static void ForEachEntity(double InBaseSeconds, double InSeconds, const FPropertyAnimatorEasingParameters& InEasingParameters, double& OutResult)
		{
			OutResult = FPropertyAnimatorEasingDoubleChannel::Evaluate(InEasingParameters, InBaseSeconds, InSeconds);
		}
	};
}

UPropertyAnimatorEasingDoubleChannelEvaluatorSystem::UPropertyAnimatorEasingDoubleChannelEvaluatorSystem(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	RelevantComponent = PropertyAnimatorComponents->EasingParameters;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Allow writing to all the possible double channels
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			DefineComponentProducer(UPropertyAnimatorEasingDoubleChannelEvaluatorSystem::StaticClass(), BuiltInComponents->DoubleResult[Index]);
		}

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBaseValueEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieScenePiecewiseDoubleBlenderSystem::StaticClass());
	}
}

void UPropertyAnimatorEasingDoubleChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* InTaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
	{
		FEntityTaskBuilder()
			.Read(BuiltInComponents->BaseValueEvalSeconds)
			.Read(BuiltInComponents->EvalSeconds)
			.Read(PropertyAnimatorComponents->EasingParameters)
			.Write(BuiltInComponents->DoubleResult[Index])
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.SetStat(GET_STATID(MovieSceneEval_EvaluatePropertyAnimatorEasingDoubleChannelTask))
			.Fork_PerEntity<FPropertyAnimatorEvaluateEasingDoubleChannels>(&Linker->EntityManager, InTaskScheduler);
	}
}

void UPropertyAnimatorEasingDoubleChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& InSubsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->BaseValueEvalSeconds)
				.Read(BuiltInComponents->BaseValueEvalSeconds)
				.Read(PropertyAnimatorComponents->EasingParameters)
				.Write(BuiltInComponents->BaseDouble[Index])
				.FilterAll({ BuiltInComponents->Tags.NeedsLink })
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.RunInline_PerEntity(&Linker->EntityManager, FPropertyAnimatorEvaluateEasingDoubleChannels());
		}
	}
	else if (Runner->GetCurrentPhase() == ESystemPhase::Evaluation)
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->BaseValueEvalSeconds)
				.Read(BuiltInComponents->EvalSeconds)
				.Read(PropertyAnimatorComponents->EasingParameters)
				.Write(BuiltInComponents->DoubleResult[Index])
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.SetStat(GET_STATID(MovieSceneEval_EvaluatePropertyAnimatorEasingDoubleChannelTask))
				.Dispatch_PerEntity<FPropertyAnimatorEvaluateEasingDoubleChannels>(&Linker->EntityManager, InPrerequisites, &InSubsequents);
		}
	}
}
