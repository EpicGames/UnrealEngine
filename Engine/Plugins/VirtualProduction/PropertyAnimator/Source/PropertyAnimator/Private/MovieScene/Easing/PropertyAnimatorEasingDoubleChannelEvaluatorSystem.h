// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "PropertyAnimatorEasingDoubleChannelEvaluatorSystem.generated.h"

/** System that is responsible for evaluating Easing channels. */
UCLASS()
class UPropertyAnimatorEasingDoubleChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	UPropertyAnimatorEasingDoubleChannelEvaluatorSystem(const FObjectInitializer& InObjectInitializer);

	//~ Begin UMovieSceneEntitySystem
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* InTaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& InSubsequents) override;
	//~ End UMovieSceneEntitySystem
};
