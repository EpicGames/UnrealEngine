// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorEasingDoubleChannelOverrideContainer.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "MovieScene/PropertyAnimatorComponentTypes.h"

bool UPropertyAnimatorEasingDoubleChannelOverrideContainer::SupportsOverride(FName InDefaultChannelTypeName) const
{
	return FMovieSceneDoubleChannel::StaticStruct()->GetFName() == InDefaultChannelTypeName;
}

void UPropertyAnimatorEasingDoubleChannelOverrideContainer::ImportEntityImpl(const UE::MovieScene::FChannelOverrideEntityImportParams& InOverrideParams
	, const UE::MovieScene::FEntityImportParams& InImportParams
	, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	TComponentTypeID<double> ResultComponent = InOverrideParams.ResultComponent.ReinterpretCast<double>();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(PropertyAnimatorComponents->EasingParameters, EasingChannel.Parameters)
		.Add(ResultComponent, TNumericLimits<double>::Max())
	);
}

const FMovieSceneChannel* UPropertyAnimatorEasingDoubleChannelOverrideContainer::GetChannel() const
{
	return &EasingChannel;
}

FMovieSceneChannel* UPropertyAnimatorEasingDoubleChannelOverrideContainer::GetChannel()
{
	return &EasingChannel;
}

#if WITH_EDITOR
FMovieSceneChannelHandle UPropertyAnimatorEasingDoubleChannelOverrideContainer::AddChannelProxy(FName InChannelName
	, FMovieSceneChannelProxyData& InProxyData
	, const FMovieSceneChannelMetaData& InMetaData)
{
	return InProxyData.AddWithDefaultEditorData(EasingChannel, InMetaData);
}
#else
void UPropertyAnimatorEasingDoubleChannelOverrideContainer::AddChannelProxy(FName InChannelName
	, FMovieSceneChannelProxyData& InProxyData)
{
	InProxyData.Add(EasingChannel);
}
#endif
