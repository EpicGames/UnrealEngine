// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelOverrideContainer.h"
#include "MovieScene/Easing/PropertyAnimatorEasingDoubleChannel.h"
#include "PropertyAnimatorEasingDoubleChannelOverrideContainer.generated.h"

UCLASS(meta=(DisplayName="Easing (Double)", ToolTip="Override a double channel to use double easing"))
class UPropertyAnimatorEasingDoubleChannelOverrideContainer : public UMovieSceneChannelOverrideContainer
{
	GENERATED_BODY()

public:
	//~ Begin UMovieSceneChannelOverrideContainer
	virtual bool SupportsOverride(FName InDefaultChannelTypeName) const override;
	virtual void ImportEntityImpl(const UE::MovieScene::FChannelOverrideEntityImportParams& InOverrideParams, const UE::MovieScene::FEntityImportParams& InImportParams, UE::MovieScene::FImportedEntity* OutImportedEntity) override;
	virtual const FMovieSceneChannel* GetChannel() const override;
	virtual FMovieSceneChannel* GetChannel() override;
#if WITH_EDITOR
	virtual FMovieSceneChannelHandle AddChannelProxy(FName InChannelName, FMovieSceneChannelProxyData& InProxyData, const FMovieSceneChannelMetaData& InMetaData) override;
#else
	virtual void AddChannelProxy(FName InChannelName, FMovieSceneChannelProxyData& InProxyData) override;
#endif
	//~ End UMovieSceneChannelOverrideContainer

private:
	UPROPERTY(EditAnywhere, Category = "Easing")
	FPropertyAnimatorEasingDoubleChannel EasingChannel;
};
