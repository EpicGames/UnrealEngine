// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorComponentTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"
#include "Templates/UniquePtr.h"

static TUniquePtr<FPropertyAnimatorComponentTypes> GPropertyAnimatorComponentTypes;
static bool GPropertyAnimatorComponentTypesDestroyed = false;

FPropertyAnimatorComponentTypes* FPropertyAnimatorComponentTypes::Get()
{
	if (!GPropertyAnimatorComponentTypes.IsValid())
	{
		check(!GPropertyAnimatorComponentTypesDestroyed);
		GPropertyAnimatorComponentTypes.Reset(new FPropertyAnimatorComponentTypes);
	}
	return GPropertyAnimatorComponentTypes.Get();
}

void FPropertyAnimatorComponentTypes::Destroy()
{
	GPropertyAnimatorComponentTypes.Reset();
	GPropertyAnimatorComponentTypesDestroyed = true;
}

FPropertyAnimatorComponentTypes::FPropertyAnimatorComponentTypes()
{
	using namespace UE::MovieScene;

	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	ComponentRegistry->NewComponentType(&WaveParameters, TEXT("Wave Parameters"));
	ComponentRegistry->NewComponentType(&EasingParameters, TEXT("Easing Parameters"));

	ComponentRegistry->Factories.DuplicateChildComponent(WaveParameters);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(WaveParameters, BuiltInComponents->EvalSeconds);

	ComponentRegistry->Factories.DuplicateChildComponent(EasingParameters);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(EasingParameters, BuiltInComponents->EvalSeconds);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(EasingParameters, BuiltInComponents->BaseValueEvalSeconds);
}
