// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieScene/Easing/PropertyAnimatorEasingParameters.h"
#include "MovieScene/Wave/PropertyAnimatorWaveParameters.h"

class UMovieSceneSection;

struct FPropertyAnimatorComponentTypes
{
	static FPropertyAnimatorComponentTypes* Get();

	static void Destroy();

	UE::MovieScene::TComponentTypeID<FPropertyAnimatorWaveParameters> WaveParameters;

	UE::MovieScene::TComponentTypeID<FPropertyAnimatorEasingParameters> EasingParameters;

private:
	FPropertyAnimatorComponentTypes();
};
