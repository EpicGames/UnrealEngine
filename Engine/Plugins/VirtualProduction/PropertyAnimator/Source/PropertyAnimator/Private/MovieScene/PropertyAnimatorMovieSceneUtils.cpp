// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/PropertyAnimatorMovieSceneUtils.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"

FFrameTime FPropertyAnimatorMovieSceneUtils::GetBaseTime(const UMovieSceneSection& InSection, const UMovieScene& InMovieScene)
{
	const TRange<FFrameNumber> PlaybackRange = InMovieScene.GetPlaybackRange();
	const TRange<FFrameNumber> SectionRange  = InSection.GetTrueRange();

	if (SectionRange.HasLowerBound())
	{
		return SectionRange.GetLowerBoundValue();
	}

	return PlaybackRange.HasLowerBound()
	? PlaybackRange.GetLowerBoundValue()
	: FFrameNumber(0);
}

double FPropertyAnimatorMovieSceneUtils::GetBaseSeconds(const UMovieSceneSection& InSection)
{
	if (const UMovieScene* MovieScene = InSection.GetTypedOuter<UMovieScene>())
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameTime BaseTime = GetBaseTime(InSection, *MovieScene);
		return TickResolution.AsSeconds(BaseTime);
	}
	return 0.0;
}
