// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScene/Wave/PropertyAnimatorWaveDoubleChannel.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "PropertyAnimatorShared.h"

double FPropertyAnimatorWaveDoubleChannel::Evaluate(double InBaseSeconds, double InSeconds) const
{
	return Evaluate(Parameters, InSeconds);
}

bool FPropertyAnimatorWaveDoubleChannel::Evaluate(const UMovieSceneSection* InSection, FFrameTime InTime, double& OutValue) const
{
	if (ensure(InSection))
	{
		if (UMovieScene* MovieScene = InSection->GetTypedOuter<UMovieScene>())
		{
			const FFrameRate TickResolution = MovieScene->GetTickResolution();
			OutValue = Evaluate(Parameters, TickResolution.AsSeconds(InTime));
			return true;
		}
	}
	return false;
}

double FPropertyAnimatorWaveDoubleChannel::Evaluate(const FPropertyAnimatorWaveParameters& InParameters, double InSeconds)
{
	return InParameters.OffsetY + UE::PropertyAnimator::Wave::Wave(InSeconds
		, InParameters.Amplitude
		, InParameters.Frequency
		, InParameters.OffsetX
		, InParameters.WaveFunction);
}
