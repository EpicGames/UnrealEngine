// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorWaveDoubleChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "MovieScene/PropertyAnimatorComponentTypes.h"
#include "MovieScene/Wave/PropertyAnimatorWaveDoubleChannel.h"
#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate Wave Double Channels"), MovieSceneEval_EvaluatePropertyAnimatorWaveDoubleChannelTask, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{
	struct FPropertyAnimatorEvaluateWaveDoubleChannels
	{
		static void ForEachEntity(double InSeconds, const FPropertyAnimatorWaveParameters& InWaveParameters, double& OutResult)
		{
			OutResult = FPropertyAnimatorWaveDoubleChannel::Evaluate(InWaveParameters, InSeconds);
		}
	};
}

UPropertyAnimatorWaveDoubleChannelEvaluatorSystem::UPropertyAnimatorWaveDoubleChannelEvaluatorSystem(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	RelevantComponent = PropertyAnimatorComponents->WaveParameters;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Allow writing to all the possible double channels
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			DefineComponentProducer(UPropertyAnimatorWaveDoubleChannelEvaluatorSystem::StaticClass(), BuiltInComponents->DoubleResult[Index]);
		}

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBaseValueEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieScenePiecewiseDoubleBlenderSystem::StaticClass());
	}
}

void UPropertyAnimatorWaveDoubleChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* InTaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
	{
		FEntityTaskBuilder()
			.Read(BuiltInComponents->EvalSeconds)
			.Read(PropertyAnimatorComponents->WaveParameters)
			.Write(BuiltInComponents->DoubleResult[Index])
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.SetStat(GET_STATID(MovieSceneEval_EvaluatePropertyAnimatorWaveDoubleChannelTask))
			.Fork_PerEntity<FPropertyAnimatorEvaluateWaveDoubleChannels>(&Linker->EntityManager, InTaskScheduler);
	}
}

void UPropertyAnimatorWaveDoubleChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& InSubsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->BaseValueEvalSeconds)
				.Read(PropertyAnimatorComponents->WaveParameters)
				.Write(BuiltInComponents->BaseDouble[Index])
				.FilterAll({ BuiltInComponents->Tags.NeedsLink })
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.RunInline_PerEntity(&Linker->EntityManager, FPropertyAnimatorEvaluateWaveDoubleChannels());
		}
	}
	else if (Runner->GetCurrentPhase() == ESystemPhase::Evaluation)
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->EvalSeconds)
				.Read(PropertyAnimatorComponents->WaveParameters)
				.Write(BuiltInComponents->DoubleResult[Index])
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.SetStat(GET_STATID(MovieSceneEval_EvaluatePropertyAnimatorWaveDoubleChannelTask))
				.Dispatch_PerEntity<FPropertyAnimatorEvaluateWaveDoubleChannels>(&Linker->EntityManager, InPrerequisites, &InSubsequents);
		}
	}
}
