// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "PropertyAnimatorWaveDoubleChannelEvaluatorSystem.generated.h"

/** System that is responsible for evaluating wave channels. */
UCLASS()
class UPropertyAnimatorWaveDoubleChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	UPropertyAnimatorWaveDoubleChannelEvaluatorSystem(const FObjectInitializer& InObjectInitializer);

	//~ Begin UMovieSceneEntitySystem
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* InTaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& InSubsequents) override;
	//~ End UMovieSceneEntitySystem
};
