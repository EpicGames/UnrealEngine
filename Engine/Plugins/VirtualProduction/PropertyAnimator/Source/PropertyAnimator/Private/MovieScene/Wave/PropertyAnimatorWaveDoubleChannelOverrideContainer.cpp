// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorWaveDoubleChannelOverrideContainer.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "MovieScene/PropertyAnimatorComponentTypes.h"

bool UPropertyAnimatorWaveDoubleChannelOverrideContainer::SupportsOverride(FName InDefaultChannelTypeName) const
{
	return FMovieSceneDoubleChannel::StaticStruct()->GetFName() == InDefaultChannelTypeName;
}

void UPropertyAnimatorWaveDoubleChannelOverrideContainer::ImportEntityImpl(const UE::MovieScene::FChannelOverrideEntityImportParams& InOverrideParams
	, const UE::MovieScene::FEntityImportParams& InImportParams
	, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FPropertyAnimatorComponentTypes* PropertyAnimatorComponents = FPropertyAnimatorComponentTypes::Get();

	TComponentTypeID<double> ResultComponent = InOverrideParams.ResultComponent.ReinterpretCast<double>();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(PropertyAnimatorComponents->WaveParameters, WaveChannel.Parameters)
		.Add(ResultComponent, TNumericLimits<double>::Max())
	);
}

const FMovieSceneChannel* UPropertyAnimatorWaveDoubleChannelOverrideContainer::GetChannel() const
{
	return &WaveChannel;
}

FMovieSceneChannel* UPropertyAnimatorWaveDoubleChannelOverrideContainer::GetChannel()
{
	return &WaveChannel;
}

#if WITH_EDITOR
FMovieSceneChannelHandle UPropertyAnimatorWaveDoubleChannelOverrideContainer::AddChannelProxy(FName InChannelName
	, FMovieSceneChannelProxyData& InProxyData
	, const FMovieSceneChannelMetaData& InMetaData)
{
	return InProxyData.AddWithDefaultEditorData(WaveChannel, InMetaData);
}
#else
void UPropertyAnimatorWaveDoubleChannelOverrideContainer::AddChannelProxy(FName InChannelName
	, FMovieSceneChannelProxyData& InProxyData)
{
	InProxyData.Add(WaveChannel);
}
#endif
