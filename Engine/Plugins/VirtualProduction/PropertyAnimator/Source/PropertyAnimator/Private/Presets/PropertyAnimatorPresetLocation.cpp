// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorPresetLocation.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "Properties/PropertyAnimatorVectorContext.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetLocation"

void UPropertyAnimatorPresetLocation::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "Location");
}

void UPropertyAnimatorPresetLocation::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	if (!InActor->GetRootComponent())
	{
		return;
	}

	const FName LocationPropertyName = USceneComponent::GetRelativeLocationPropertyName();
	FProperty* LocationProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), LocationPropertyName);
	check(LocationProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(InActor->GetRootComponent(), LocationProperty, nullptr));
}

void UPropertyAnimatorPresetLocation::OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties)
{
	Super::OnPresetApplied(InAnimator, InProperties);

	for (const FPropertyAnimatorCoreData& Property : InProperties)
	{
		if (UPropertyAnimatorFloatContext* FloatContext = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorFloatContext>(Property))
		{
			FloatContext->SetMode(EPropertyAnimatorCoreMode::Additive);
			FloatContext->SetAmplitudeMin(-100.f);
			FloatContext->SetAmplitudeMax(100.f);
		}
		else if (UPropertyAnimatorVectorContext* VectorContext = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorVectorContext>(Property))
		{
			VectorContext->SetMode(EPropertyAnimatorCoreMode::Additive);
			VectorContext->SetAmplitudeMin(FVector(0, 0, -100.f));
			VectorContext->SetAmplitudeMax(FVector(0, 0, 100.f));
		}
	}
}

#undef LOCTEXT_NAMESPACE
