// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Presets/PropertyAnimatorCorePropertyPreset.h"
#include "PropertyAnimatorPresetLocation.generated.h"

class AActor;
class UPropertyAnimatorCoreBase;

/**
 * Preset for position properties (X, Y, Z) on scene component
 */
UCLASS(Transient)
class UPropertyAnimatorPresetLocation : public UPropertyAnimatorCorePropertyPreset
{
	GENERATED_BODY()

public:
	UPropertyAnimatorPresetLocation()
	{
		PresetName = TEXT("Location");
	}

protected:
	//~ Begin UPropertyAnimatorCorePresetBase
	virtual void OnPresetRegistered() override;
	virtual void GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const override;
	virtual void OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties) override;
	virtual bool LoadPreset() override { return true; }
	//~ End UPropertyAnimatorCorePresetBase
};