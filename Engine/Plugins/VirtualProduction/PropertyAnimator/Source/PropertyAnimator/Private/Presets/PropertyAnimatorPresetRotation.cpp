// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetRotation.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "Properties/PropertyAnimatorRotatorContext.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetRotation"

void UPropertyAnimatorPresetRotation::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "Rotation");
}

void UPropertyAnimatorPresetRotation::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	if (!InActor->GetRootComponent())
	{
		return;
	}

	const FName RotationPropertyName = USceneComponent::GetRelativeRotationPropertyName();
	FProperty* RotationProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), RotationPropertyName);
	check(RotationProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(InActor->GetRootComponent(), RotationProperty, nullptr));
}

void UPropertyAnimatorPresetRotation::OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties)
{
	Super::OnPresetApplied(InAnimator, InProperties);

	for (const FPropertyAnimatorCoreData& Property : InProperties)
	{
		if (UPropertyAnimatorFloatContext* Context = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorFloatContext>(Property))
		{
			Context->SetMode(EPropertyAnimatorCoreMode::Absolute);
			Context->SetAmplitudeMin(-90.f);
			Context->SetAmplitudeMax(90.f);
		}
		else if (UPropertyAnimatorRotatorContext* RotatorContext = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorRotatorContext>(Property))
		{
			RotatorContext->SetMode(EPropertyAnimatorCoreMode::Absolute);
			RotatorContext->SetAmplitudeMin(FRotator(0, -90, 0));
			RotatorContext->SetAmplitudeMax(FRotator(0, 90, 0));
		}
	}
}

#undef LOCTEXT_NAMESPACE
