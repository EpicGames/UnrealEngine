// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Presets/PropertyAnimatorCorePropertyPreset.h"
#include "PropertyAnimatorPresetRotation.generated.h"

class AActor;
class UPropertyAnimatorCoreBase;

/**
 * Preset for position properties (Roll, Pitch, Yaw) on scene component
 */
UCLASS(Transient)
class UPropertyAnimatorPresetRotation : public UPropertyAnimatorCorePropertyPreset
{
	GENERATED_BODY()

public:
	UPropertyAnimatorPresetRotation()
	{
		PresetName = TEXT("Rotation");
	}

protected:
	//~ Begin UPropertyAnimatorCorePresetBase
	virtual void OnPresetRegistered() override;
	virtual void GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const override;
	virtual void OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties) override;
	virtual bool LoadPreset() override { return true; }
	//~ End UPropertyAnimatorCorePresetBase
};