// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetScale.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "Properties/PropertyAnimatorVectorContext.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetScale"

void UPropertyAnimatorPresetScale::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "Scale");
}

void UPropertyAnimatorPresetScale::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	if (!InActor->GetRootComponent())
	{
		return;
	}

	const FName ScalePropertyName = USceneComponent::GetRelativeScale3DPropertyName();
	FProperty* ScaleProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), ScalePropertyName);
	check(ScaleProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(InActor->GetRootComponent(), ScaleProperty, nullptr));
}

void UPropertyAnimatorPresetScale::OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties)
{
	Super::OnPresetApplied(InAnimator, InProperties);

	for (const FPropertyAnimatorCoreData& Property : InProperties)
	{
		if (UPropertyAnimatorFloatContext* Context = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorFloatContext>(Property))
		{
			Context->SetMode(EPropertyAnimatorCoreMode::Absolute);
			Context->SetAmplitudeMin(0.f);
			Context->SetAmplitudeMax(1.f);
		}
		else if (UPropertyAnimatorVectorContext* VectorContext = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorVectorContext>(Property))
		{
			VectorContext->SetMode(EPropertyAnimatorCoreMode::Absolute);
			VectorContext->SetAmplitudeMin(FVector::ZeroVector);
			VectorContext->SetAmplitudeMax(FVector::OneVector);
		}
	}
}

#undef LOCTEXT_NAMESPACE
