// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetText.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Properties/PropertyAnimatorCoreContext.h"
#include "Text3DComponent.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetText"

void UPropertyAnimatorPresetText::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "Text");
}

void UPropertyAnimatorPresetText::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	const UText3DComponent* TextComponent = InActor->FindComponentByClass<UText3DComponent>();
	if (!TextComponent)
	{
		return;
	}

	static const FName TextPropertyName(TEXT("Text"));
	FProperty* TextProperty = FindFProperty<FProperty>(UText3DComponent::StaticClass(), TextPropertyName);
	check(TextProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(InActor->GetRootComponent(), TextProperty, nullptr));
}

void UPropertyAnimatorPresetText::OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties)
{
	Super::OnPresetApplied(InAnimator, InProperties);

	for (const FPropertyAnimatorCoreData& Property : InProperties)
	{
		if (UPropertyAnimatorCoreContext* Context = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorCoreContext>(Property))
		{
			Context->SetMode(EPropertyAnimatorCoreMode::Absolute);
		}
	}
}

#undef LOCTEXT_NAMESPACE
