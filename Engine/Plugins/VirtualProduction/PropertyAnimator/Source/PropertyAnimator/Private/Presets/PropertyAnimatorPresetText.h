// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Presets/PropertyAnimatorCorePropertyPreset.h"
#include "PropertyAnimatorPresetText.generated.h"

class AActor;
class UPropertyAnimatorCoreBase;

/**
 * Preset for text property on root text 3d component
 */
UCLASS(Transient)
class UPropertyAnimatorPresetText : public UPropertyAnimatorCorePropertyPreset
{
	GENERATED_BODY()

public:
	UPropertyAnimatorPresetText()
	{
		PresetName = TEXT("Text");
	}

protected:
	//~ Begin UPropertyAnimatorCorePresetBase
	virtual void OnPresetRegistered() override;
	virtual void GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const override;
	virtual void OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties) override;
	virtual bool LoadPreset() override { return true; }
	//~ End UPropertyAnimatorCorePresetBase
};