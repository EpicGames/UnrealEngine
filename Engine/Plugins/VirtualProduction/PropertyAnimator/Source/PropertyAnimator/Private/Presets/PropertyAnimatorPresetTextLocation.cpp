// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetTextLocation.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Characters/Text3DCharacterBase.h"
#include "Properties/PropertyAnimatorTextResolver.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "Text3DComponent.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetTextLocation"

void UPropertyAnimatorPresetTextLocation::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "TextCharacterLocation");
}

void UPropertyAnimatorPresetTextLocation::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	UText3DComponent* TextComponent = InActor->FindComponentByClass<UText3DComponent>();
	if (!TextComponent)
	{
		return;
	}

	const FName LocationPropertyName = UText3DCharacterBase::GetRelativeLocationPropertyName();
	FProperty* LocationProperty = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), LocationPropertyName);
	check(LocationProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(TextComponent, LocationProperty, nullptr, UPropertyAnimatorTextResolver::StaticClass()));
}

#undef LOCTEXT_NAMESPACE
