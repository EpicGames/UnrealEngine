// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetTextRotation.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Characters/Text3DCharacterBase.h"
#include "Properties/PropertyAnimatorTextResolver.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "Text3DComponent.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetTextRotation"

void UPropertyAnimatorPresetTextRotation::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "TextCharacterRotation");
}

void UPropertyAnimatorPresetTextRotation::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	UText3DComponent* TextComponent = InActor->FindComponentByClass<UText3DComponent>();
	if (!TextComponent)
	{
		return;
	}

	const FName RotationPropertyName = UText3DCharacterBase::GetRelativeRotationPropertyName();
	FProperty* RotationProperty = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), RotationPropertyName);
	check(RotationProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(TextComponent, RotationProperty, nullptr, UPropertyAnimatorTextResolver::StaticClass()));
}

#undef LOCTEXT_NAMESPACE
