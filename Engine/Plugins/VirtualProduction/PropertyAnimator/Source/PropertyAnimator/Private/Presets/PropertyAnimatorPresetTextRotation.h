// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorPresetRotation.h"
#include "PropertyAnimatorPresetTextRotation.generated.h"

class AActor;
class UPropertyAnimatorCoreBase;

/**
 * Preset for text character position properties (Roll, Pitch, Yaw) on scene component
 */
UCLASS()
class UPropertyAnimatorPresetTextRotation : public UPropertyAnimatorPresetRotation
{
	GENERATED_BODY()

public:
	UPropertyAnimatorPresetTextRotation()
	{
		PresetName = TEXT("TextCharacterRotation");
	}

protected:
	//~ Begin UPropertyAnimatorCorePresetBase
	virtual void OnPresetRegistered() override;
	virtual void GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const override;
	//~ End UPropertyAnimatorCorePresetBase
};