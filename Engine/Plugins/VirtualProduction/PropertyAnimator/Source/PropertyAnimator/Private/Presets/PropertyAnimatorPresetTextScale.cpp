// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetTextScale.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Characters/Text3DCharacterBase.h"
#include "Properties/PropertyAnimatorTextResolver.h"
#include "Properties/PropertyAnimatorFloatContext.h"
#include "Text3DComponent.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetTextScale"

void UPropertyAnimatorPresetTextScale::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "TextCharacterScale");
}

void UPropertyAnimatorPresetTextScale::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	UText3DComponent* TextComponent = InActor->FindComponentByClass<UText3DComponent>();
	if (!TextComponent)
	{
		return;
	}

	const FName ScalePropertyName = UText3DCharacterBase::GetRelativeScalePropertyName();
	FProperty* ScaleProperty = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), ScalePropertyName);
	check(ScaleProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(TextComponent, ScaleProperty, nullptr, UPropertyAnimatorTextResolver::StaticClass()));
}

#undef LOCTEXT_NAMESPACE
