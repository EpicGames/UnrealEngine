// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorPresetScale.h"
#include "PropertyAnimatorPresetTextScale.generated.h"

class AActor;
class UPropertyAnimatorCoreBase;

/**
 * Preset for text character position properties (X, Y, Z) on scene component
 */
UCLASS()
class UPropertyAnimatorPresetTextScale : public UPropertyAnimatorPresetScale
{
	GENERATED_BODY()

public:
	UPropertyAnimatorPresetTextScale()
	{
		PresetName = TEXT("TextCharacterScale");
	}

protected:
	//~ Begin UPropertyAnimatorCorePresetBase
	virtual void OnPresetRegistered() override;
	virtual void GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const override;
	//~ End UPropertyAnimatorCorePresetBase
};