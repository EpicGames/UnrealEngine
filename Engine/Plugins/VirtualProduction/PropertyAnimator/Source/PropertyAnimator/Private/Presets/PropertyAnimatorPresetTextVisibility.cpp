// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetTextVisibility.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Characters/Text3DCharacterBase.h"
#include "Properties/PropertyAnimatorTextResolver.h"
#include "Text3DComponent.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetTextVisibility"

void UPropertyAnimatorPresetTextVisibility::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "TextCharacterVisibility");
}

void UPropertyAnimatorPresetTextVisibility::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	UText3DComponent* TextComponent = InActor->FindComponentByClass<UText3DComponent>();
	if (!TextComponent)
	{
		return;
	}

	const FName VisibilityPropertyName = UText3DCharacterBase::GetVisiblePropertyName();
	FProperty* VisibilityProperty = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), VisibilityPropertyName);
	check(VisibilityProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(TextComponent, VisibilityProperty, nullptr, UPropertyAnimatorTextResolver::StaticClass()));
}

#undef LOCTEXT_NAMESPACE
