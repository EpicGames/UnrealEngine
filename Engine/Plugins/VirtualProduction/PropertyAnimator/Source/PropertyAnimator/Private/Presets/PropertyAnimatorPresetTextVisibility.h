// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorPresetVisibility.h"
#include "PropertyAnimatorPresetTextVisibility.generated.h"

class AActor;
class UPropertyAnimatorCoreBase;

/**
 * Preset for text character visibility on scene component
 */
UCLASS()
class UPropertyAnimatorPresetTextVisibility : public UPropertyAnimatorPresetVisibility
{
	GENERATED_BODY()

public:
	UPropertyAnimatorPresetTextVisibility()
	{
		PresetName = TEXT("TextCharacterVisibility");
	}

protected:
	//~ Begin UPropertyAnimatorCorePresetBase
	virtual void OnPresetRegistered() override;
	virtual void GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const override;
	//~ End UPropertyAnimatorCorePresetBase
};