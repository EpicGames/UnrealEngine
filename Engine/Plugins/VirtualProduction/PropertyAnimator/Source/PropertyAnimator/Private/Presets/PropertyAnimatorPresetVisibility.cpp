// Copyright Epic Games, Inc. All Rights Reserved.

#include "Presets/PropertyAnimatorPresetVisibility.h"

#include "Animators/PropertyAnimatorCoreBase.h"
#include "Properties/Converters/PropertyAnimatorCoreConverterTraits.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorPresetVisibility"

void UPropertyAnimatorPresetVisibility::OnPresetRegistered()
{
	Super::OnPresetRegistered();

	PresetDisplayName = LOCTEXT("PresetDisplayName", "Visibility");
}

void UPropertyAnimatorPresetVisibility::GetPresetProperties(const AActor* InActor, const UPropertyAnimatorCoreBase* InAnimator, TSet<FPropertyAnimatorCoreData>& OutProperties) const
{
	if (!InActor->GetRootComponent())
	{
		return;
	}

	const FName VisibilityPropertyName = USceneComponent::GetVisiblePropertyName();
	FProperty* VisibilityProperty = FindFProperty<FProperty>(USceneComponent::StaticClass(), VisibilityPropertyName);
	check(VisibilityProperty);

	OutProperties.Add(FPropertyAnimatorCoreData(InActor->GetRootComponent(), VisibilityProperty, nullptr));
}

void UPropertyAnimatorPresetVisibility::OnPresetApplied(UPropertyAnimatorCoreBase* InAnimator, const TSet<FPropertyAnimatorCoreData>& InProperties)
{
	Super::OnPresetApplied(InAnimator, InProperties);

	for (const FPropertyAnimatorCoreData& Property : InProperties)
	{
		if (UPropertyAnimatorCoreContext* Context = InAnimator->GetLinkedPropertyContext<UPropertyAnimatorCoreContext>(Property))
		{
			if (FBoolConverterRule* ConverterRule = Context->GetConverterRule<FBoolConverterRule>())
			{
				ConverterRule->TrueConditions.Empty();

				FBoolConverterCondition NewCondition(EBoolConverterComparison::Greater, 0);
				ConverterRule->TrueConditions.Add(NewCondition);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
