// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/PropertyAnimatorFloatContext.h"

#include "Animators/PropertyAnimatorCoreBase.h"

void UPropertyAnimatorFloatContext::SetAmplitudeMin(double InAmplitude)
{
	AmplitudeMin = GetClampedAmplitude(InAmplitude);
}

void UPropertyAnimatorFloatContext::SetAmplitudeMax(double InAmplitude)
{
	AmplitudeMax = GetClampedAmplitude(InAmplitude);
}

#if WITH_EDITOR
void UPropertyAnimatorFloatContext::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorFloatContext, AmplitudeMin))
	{
		SetAmplitudeMin(AmplitudeMin);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorFloatContext, AmplitudeMax))
	{
		SetAmplitudeMax(AmplitudeMax);
	}
}
#endif

bool UPropertyAnimatorFloatContext::EvaluateProperty(const FPropertyAnimatorCoreData& InProperty, const FInstancedPropertyBag& InAnimatorResult, FInstancedPropertyBag& OutEvaluatedValues)
{
	const TValueOrError<float, EPropertyBagResult> AlphaResult = InAnimatorResult.GetValueFloat(UPropertyAnimatorCoreBase::AlphaParameterName);
	const TValueOrError<float, EPropertyBagResult> MagnitudeResult = InAnimatorResult.GetValueFloat(UPropertyAnimatorCoreBase::MagnitudeParameterName);

	if (AlphaResult.HasValue())
	{
		const FName PropertyHash = InProperty.GetLocatorPathHash();
		OutEvaluatedValues.AddProperty(PropertyHash, EPropertyBagPropertyType::Double);
		OutEvaluatedValues.SetValueFloat(PropertyHash, MagnitudeResult.GetValue() * FMath::Lerp(AmplitudeMin, AmplitudeMax, AlphaResult.GetValue()));
		return true;
	}

	return false;
}

void UPropertyAnimatorFloatContext::OnAnimatedPropertyLinked()
{
	Super::OnAnimatedPropertyLinked();

	AmplitudeClampMin.Reset();
	AmplitudeClampMax.Reset();

#if WITH_EDITOR
	const FPropertyAnimatorCoreData& Property = GetAnimatedProperty();
	const FProperty* LeafProperty = Property.GetLeafProperty();

	checkf(LeafProperty, TEXT("Animated leaf property must be valid"))

	// Assign Min and Max value based on editor meta data available
	if (LeafProperty->IsA<FNumericProperty>())
	{
		if (LeafProperty->HasMetaData(TEXT("ClampMin")))
		{
			AmplitudeMin = LeafProperty->GetFloatMetaData(FName("ClampMin"));
			AmplitudeClampMin = AmplitudeMin;
		}
		else if (LeafProperty->HasMetaData(TEXT("UIMin")))
		{
			AmplitudeMin = LeafProperty->GetFloatMetaData(FName("UIMin"));
		}

		if (LeafProperty->HasMetaData(TEXT("ClampMax")))
		{
			AmplitudeMax =  LeafProperty->GetFloatMetaData(FName("ClampMax"));
			AmplitudeClampMax = AmplitudeMax;
		}
		else if (LeafProperty->HasMetaData(TEXT("UIMax")))
		{
			AmplitudeMax =  LeafProperty->GetFloatMetaData(FName("UIMax"));
		}
	}
#endif
}

bool UPropertyAnimatorFloatContext::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ContextArchive = InValue->AsMutableObject();

		double AmplitudeMinValue = AmplitudeMin;
		ContextArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorFloatContext, AmplitudeMin), AmplitudeMinValue);
		SetAmplitudeMin(AmplitudeMinValue);

		double AmplitudeMaxValue = AmplitudeMax;
		ContextArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorFloatContext, AmplitudeMax), AmplitudeMaxValue);
		SetAmplitudeMax(AmplitudeMaxValue);

		return true;
	}

	return false;
}

bool UPropertyAnimatorFloatContext::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ContextArchive = OutValue->AsMutableObject();

		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorFloatContext, AmplitudeMin), AmplitudeMin);
		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorFloatContext, AmplitudeMax), AmplitudeMax);

		return true;
	}

	return false;
}

double UPropertyAnimatorFloatContext::GetClampedAmplitude(double InAmplitude)
{
	if (AmplitudeClampMin.IsSet())
	{
		InAmplitude = FMath::Max(InAmplitude, AmplitudeClampMin.GetValue());
	}

	if (AmplitudeClampMax.IsSet())
	{
		InAmplitude = FMath::Min(InAmplitude, AmplitudeClampMax.GetValue());
	}

	return InAmplitude;
}
