// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/PropertyAnimatorRotatorContext.h"

#include "Animators/PropertyAnimatorCoreBase.h"

void UPropertyAnimatorRotatorContext::SetAmplitudeMin(const FRotator& InAmplitude)
{
	AmplitudeMin = GetClampedAmplitude(InAmplitude);
}

void UPropertyAnimatorRotatorContext::SetAmplitudeMax(const FRotator& InAmplitude)
{
	AmplitudeMax = GetClampedAmplitude(InAmplitude);
}

#if WITH_EDITOR
void UPropertyAnimatorRotatorContext::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorRotatorContext, AmplitudeMin))
	{
		SetAmplitudeMin(AmplitudeMin);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorRotatorContext, AmplitudeMax))
	{
		SetAmplitudeMax(AmplitudeMax);
	}
}
#endif

bool UPropertyAnimatorRotatorContext::EvaluateProperty(const FPropertyAnimatorCoreData& InProperty, const FInstancedPropertyBag& InAnimatorResult, FInstancedPropertyBag& OutEvaluatedValues)
{
	const TValueOrError<float, EPropertyBagResult> AlphaResult = InAnimatorResult.GetValueFloat(UPropertyAnimatorCoreBase::AlphaParameterName);
	const TValueOrError<float, EPropertyBagResult> MagnitudeResult = InAnimatorResult.GetValueFloat(UPropertyAnimatorCoreBase::MagnitudeParameterName);

	if (AlphaResult.HasValue())
	{
		const FName PropertyHash = InProperty.GetLocatorPathHash();
		OutEvaluatedValues.AddProperty(PropertyHash, EPropertyBagPropertyType::Struct, TBaseStructure<FRotator>::Get());
		OutEvaluatedValues.SetValueStruct(PropertyHash, MagnitudeResult.GetValue() * FMath::LerpRange(AmplitudeMin, AmplitudeMax, AlphaResult.GetValue()));
		return true;
	}

	return false;
}

void UPropertyAnimatorRotatorContext::OnAnimatedPropertyLinked()
{
	Super::OnAnimatedPropertyLinked();

	AmplitudeClampMin.Reset();
	AmplitudeClampMax.Reset();

#if WITH_EDITOR
	const FPropertyAnimatorCoreData& Property = GetAnimatedProperty();
	const FProperty* LeafProperty = Property.GetLeafProperty();

	checkf(LeafProperty, TEXT("Animated leaf property must be valid"))

	// Assign Min and Max value based on editor meta data available
	if (LeafProperty->IsA<FStructProperty>())
	{
		if (LeafProperty->HasMetaData(TEXT("ClampMin")))
		{
			AmplitudeMin = FRotator(LeafProperty->GetFloatMetaData(FName("ClampMin")));
			AmplitudeClampMin = AmplitudeMin;
		}
		else if (LeafProperty->HasMetaData(TEXT("UIMin")))
		{
			AmplitudeMin = FRotator(LeafProperty->GetFloatMetaData(FName("UIMin")));
		}

		if (LeafProperty->HasMetaData(TEXT("ClampMax")))
		{
			AmplitudeMax = FRotator(LeafProperty->GetFloatMetaData(FName("ClampMax")));
			AmplitudeClampMax = AmplitudeMax;
		}
		else if (LeafProperty->HasMetaData(TEXT("UIMax")))
		{
			AmplitudeMax = FRotator(LeafProperty->GetFloatMetaData(FName("UIMax")));
		}
	}
#endif
}

bool UPropertyAnimatorRotatorContext::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ContextArchive = InValue->AsMutableObject();

		FString AmplitudeMinValue = AmplitudeMin.ToString();
		ContextArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorRotatorContext, AmplitudeMin), AmplitudeMinValue);
		FRotator ParseAmplitudeMin;
		ParseAmplitudeMin.InitFromString(AmplitudeMinValue);
		SetAmplitudeMin(ParseAmplitudeMin);

		FString AmplitudeMaxValue = AmplitudeMax.ToString();
		ContextArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorRotatorContext, AmplitudeMax), AmplitudeMaxValue);
		FRotator ParseAmplitudeMax;
		ParseAmplitudeMax.InitFromString(AmplitudeMaxValue);
		SetAmplitudeMax(ParseAmplitudeMax);

		return true;
	}

	return false;
}

bool UPropertyAnimatorRotatorContext::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ContextArchive = OutValue->AsMutableObject();

		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorRotatorContext, AmplitudeMin), AmplitudeMin.ToString());
		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorRotatorContext, AmplitudeMax), AmplitudeMax.ToString());

		return true;
	}

	return false;
}

FRotator UPropertyAnimatorRotatorContext::GetClampedAmplitude(FRotator InAmplitude)
{
	if (AmplitudeClampMin.IsSet())
	{
		const FRotator& MinAmplitude = AmplitudeClampMin.GetValue();

		InAmplitude.Roll = FMath::Max(InAmplitude.Roll, MinAmplitude.Roll);
		InAmplitude.Pitch = FMath::Max(InAmplitude.Pitch, MinAmplitude.Pitch);
		InAmplitude.Yaw = FMath::Max(InAmplitude.Yaw, MinAmplitude.Yaw);
	}

	if (AmplitudeClampMax.IsSet())
	{
		const FRotator& MaxAmplitude = AmplitudeClampMax.GetValue();

		InAmplitude.Roll = FMath::Min(InAmplitude.Roll, MaxAmplitude.Roll);
		InAmplitude.Pitch = FMath::Min(InAmplitude.Pitch, MaxAmplitude.Pitch);
		InAmplitude.Yaw = FMath::Min(InAmplitude.Yaw, MaxAmplitude.Yaw);
	}

	return InAmplitude;
}
