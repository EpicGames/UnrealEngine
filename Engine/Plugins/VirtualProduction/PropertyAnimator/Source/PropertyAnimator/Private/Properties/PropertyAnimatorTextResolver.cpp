// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/PropertyAnimatorTextResolver.h"

#include "Characters/Text3DCharacterBase.h"
#include "Misc/EnumerateRange.h"
#include "Presets/PropertyAnimatorCorePresetArchive.h"
#include "Text3DComponent.h"

void UPropertyAnimatorTextResolver::SetUnit(EPropertyAnimatorTextResolverRangeUnit InUnit)
{
	Unit = InUnit;
}

void UPropertyAnimatorTextResolver::SetPercentageRange(float InPercentageRange)
{
	PercentageRange = FMath::Clamp(InPercentageRange, 0.f, 100.f);
}

void UPropertyAnimatorTextResolver::SetPercentageOffset(float InPercentageOffset)
{
	PercentageOffset = InPercentageOffset;
}

void UPropertyAnimatorTextResolver::SetCharacterRangeCount(int32 InCharacterRangeCount)
{
	CharacterRangeCount = FMath::Max(InCharacterRangeCount, 0);
}

void UPropertyAnimatorTextResolver::SetCharacterOffsetCount(int32 InCharacterOffsetCount)
{
	CharacterOffsetCount = InCharacterOffsetCount;
}

void UPropertyAnimatorTextResolver::SetWordRangeCount(int32 InWordRangeCount)
{
	WordRangeCount = FMath::Max(InWordRangeCount, 0);
}

void UPropertyAnimatorTextResolver::SetWordOffsetCount(int32 InWordOffsetCount)
{
	WordOffsetCount = InWordOffsetCount;
}

void UPropertyAnimatorTextResolver::SetDirection(EPropertyAnimatorTextResolverRangeDirection InDirection)
{
	Direction = InDirection;
}

bool UPropertyAnimatorTextResolver::FixUpProperty(FPropertyAnimatorCoreData& InOldProperty)
{
	if (InOldProperty.GetPropertyResolver() != this)
	{
		return false;
	}

	TSet<FPropertyAnimatorCoreData> TemplateProperties;
	const TArray<FName> SearchPaths = InOldProperty.GetChainPropertyNames();
	GetTemplateProperties(InOldProperty.GetOwningActor(), TemplateProperties, &SearchPaths);

	if (TemplateProperties.Num() != 1)
	{
		return false;
	}

	FPropertyAnimatorCoreData TemplateProperty = TemplateProperties.Array()[0];
	if (TemplateProperty == InOldProperty)
	{
		return false;
	}

	InOldProperty = TemplateProperty;
	return true;
}

void UPropertyAnimatorTextResolver::GetTemplateProperties(UObject* InContext, TSet<FPropertyAnimatorCoreData>& OutProperties, const TArray<FName>* InSearchPath)
{
	const AActor* Actor = Cast<AActor>(InContext);
	if (!Actor)
	{
		return;
	}

	UText3DComponent* TextComponent = Actor->FindComponentByClass<UText3DComponent>();
	if (!TextComponent)
	{
		return;
	}

	TArray<FPropertyAnimatorCoreData> TemplateProperties;
	TemplateProperties.Reserve(4);

	FProperty* RelativeLocation = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), UText3DCharacterBase::GetRelativeLocationPropertyName());
	FPropertyAnimatorCoreData RelativeLocationProperty(TextComponent, RelativeLocation, nullptr, GetClass());
	TemplateProperties.Add(RelativeLocationProperty);

	FProperty* RelativeRotation = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), UText3DCharacterBase::GetRelativeRotationPropertyName());
	FPropertyAnimatorCoreData RelativeRotationProperty(TextComponent, RelativeRotation, nullptr, GetClass());
	TemplateProperties.Add(RelativeRotationProperty);

	FProperty* RelativeScale = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), UText3DCharacterBase::GetRelativeScalePropertyName());
	FPropertyAnimatorCoreData RelativeScaleProperty(TextComponent, RelativeScale, nullptr, GetClass());
	TemplateProperties.Add(RelativeScaleProperty);

	FProperty* Visible = FindFProperty<FProperty>(UText3DCharacterBase::StaticClass(), UText3DCharacterBase::GetVisiblePropertyName());
	FPropertyAnimatorCoreData VisibleProperty(TextComponent, Visible, nullptr, GetClass());
	TemplateProperties.Add(VisibleProperty);

	if (!InSearchPath || InSearchPath->IsEmpty())
	{
		OutProperties.Append(TemplateProperties);
		return;
	}

	while (!TemplateProperties.IsEmpty())
	{
		FPropertyAnimatorCoreData TemplateProperty = TemplateProperties.Pop();
		const TArray<FName> ChainPropertyNames = TemplateProperty.GetChainPropertyNames();
		for (TConstEnumerateRef<FName> TemplatePathPart : EnumerateRange(ChainPropertyNames))
		{
			if (!InSearchPath->IsValidIndex(TemplatePathPart.GetIndex()))
			{
				break;
			}

			const FName& SearchPathPart = (*InSearchPath)[TemplatePathPart.GetIndex()];
			if (SearchPathPart != (*TemplatePathPart))
			{
				break;
			}

			if (ChainPropertyNames.Num() < InSearchPath->Num())
			{
				int32 DepthSearch = InSearchPath->Num() - ChainPropertyNames.Num();
				TemplateProperties.Append(TemplateProperty.GetChildrenProperties(DepthSearch));
				break;
			}

			if (TemplatePathPart.GetIndex() == InSearchPath->Num() - 1)
			{
				OutProperties.Add(TemplateProperty);
				break;
			}
		}
	}
}

void UPropertyAnimatorTextResolver::ResolveTemplateProperties(const FPropertyAnimatorCoreData& InTemplateProperty, TArray<FPropertyAnimatorCoreData>& OutProperties, bool bInForEvaluation)
{
	if (!InTemplateProperty.IsResolvable())
	{
		return;
	}

	const UText3DComponent* TextComponent = Cast<UText3DComponent>(InTemplateProperty.GetOwningComponent());
	if (!TextComponent)
	{
		return;
	}

	const TArray<FProperty*> ChainProperties = InTemplateProperty.GetChainProperties();

	// Gather each character in the text
	TextComponent->ForEachCharacter([this, &OutProperties, &ChainProperties](UText3DCharacterBase* InCharacter, uint16 InIndex, uint16 InCount)
	{
		FPropertyAnimatorCoreData CharacterProperty(InCharacter, ChainProperties);
		OutProperties.Emplace(CharacterProperty);
	});

	if (!bInForEvaluation || OutProperties.IsEmpty())
	{
		return;
	}

	const int32 MaxIndex = OutProperties.Num();
	int32 BeginIndex = 0;
	int32 EndIndex = 0;

	switch (Unit)
	{
		case EPropertyAnimatorTextResolverRangeUnit::Percentage:
		{
			const float Range = PercentageRange / 100.f;
			const float Offset = PercentageOffset / 100.f;

			const int32 RangeCount = FMath::Clamp(FMath::RoundToInt(MaxIndex * Range), 0, MaxIndex);
			int32 RangeOffset = FMath::RoundToInt(MaxIndex * Offset);

			int32 CharacterStart = 0;
			int32 CharacterEnd = RangeCount;

			if (Direction == EPropertyAnimatorTextResolverRangeDirection::RightToLeft)
			{
				CharacterStart = MaxIndex - RangeCount;
				CharacterEnd = MaxIndex;
				RangeOffset *= -1;
			}
			else if (Direction == EPropertyAnimatorTextResolverRangeDirection::FromCenter)
			{
				const int32 CharacterMid = MaxIndex / 2;
				const int32 ExpansionLeft = RangeCount / 2;
				const int32 ExpansionRight = RangeCount - ExpansionLeft;
				CharacterStart = CharacterMid - ExpansionLeft;
				CharacterEnd = CharacterMid + ExpansionRight;
			}

			BeginIndex = CharacterStart + RangeOffset;
			EndIndex = CharacterEnd + RangeOffset;
		}
		break;

		case EPropertyAnimatorTextResolverRangeUnit::Character:
		{
			const int32 RangeCount = CharacterRangeCount;
			int32 RangeOffset = CharacterOffsetCount;
			int32 CharacterStart = 0;
			int32 CharacterEnd = CharacterRangeCount - 1;

			if (Direction == EPropertyAnimatorTextResolverRangeDirection::RightToLeft)
			{
				CharacterStart = MaxIndex - RangeCount;
				CharacterEnd = MaxIndex;
				RangeOffset *= -1;
			}
			else if (Direction == EPropertyAnimatorTextResolverRangeDirection::FromCenter)
			{
				const int32 CharacterMid = MaxIndex / 2;
				const int32 ExpansionLeft = RangeCount / 2;
				const int32 ExpansionRight = RangeCount - ExpansionLeft;
				CharacterStart = CharacterMid - ExpansionLeft;
				CharacterEnd = CharacterMid + ExpansionRight;
			}

			BeginIndex = CharacterStart + RangeOffset;
			EndIndex = CharacterEnd + RangeOffset;
		}
		break;

		case EPropertyAnimatorTextResolverRangeUnit::Word:
		{
			const FText3DStatistics& TextStats = TextComponent->GetStatistics();

			if (TextStats.Words.IsEmpty())
			{
				break;
			}

			const int32 WordCount = TextStats.Words.Num();
			const int32 RangeCount = FMath::Clamp(WordRangeCount, 0, WordCount);
			int32 WordOffset = WordOffsetCount;

			int32 WordStart = 0;
			int32 WordEnd = RangeCount - 1;

			if (Direction == EPropertyAnimatorTextResolverRangeDirection::RightToLeft)
			{
				WordStart = WordCount - RangeCount;
				WordEnd = WordCount - 1;
				WordOffset *= -1;
			}
			else if (Direction == EPropertyAnimatorTextResolverRangeDirection::FromCenter)
			{
				const int32 WordMid = WordCount / 2;
				const int32 ExpansionLeft = RangeCount / 2;
				const int32 ExpansionRight = RangeCount - ExpansionLeft;

				WordStart = WordMid - ExpansionLeft;
				WordEnd = WordMid + ExpansionRight - 1;
			}

			WordStart += WordOffset;
			WordEnd += WordOffset;

			if (TextStats.Words.IsValidIndex(WordStart))
			{
				BeginIndex = TextStats.Words[WordStart].RenderRange.BeginIndex;
			}

			if (TextStats.Words.IsValidIndex(WordEnd))
			{
				EndIndex = TextStats.Words[WordEnd].RenderRange.EndIndex;
			}
			else if (WordEnd >= WordCount && WordStart < WordCount)
			{
				EndIndex = TextStats.Words.Last().RenderRange.EndIndex;
			}
		}
		break;
	}

	if (EndIndex < 0 || BeginIndex > EndIndex || BeginIndex == EndIndex || BeginIndex > MaxIndex)
	{
		OutProperties.Empty();
		return;
	}

	// Remove at the end
	if (EndIndex < MaxIndex)
	{
		OutProperties.RemoveAt(EndIndex, MaxIndex - EndIndex);
	}

	// Remove at the start
	if (BeginIndex > 0 && BeginIndex <= MaxIndex)
	{
		OutProperties.RemoveAt(0, BeginIndex);
	}
}

bool UPropertyAnimatorTextResolver::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ResolverArchive = InValue->AsMutableObject();

		uint64 UnitValue = static_cast<uint64>(Unit);
		ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, Unit), UnitValue);
		SetUnit(static_cast<EPropertyAnimatorTextResolverRangeUnit>(UnitValue));

		{
			double PercentageRangeValue = PercentageRange;
			ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, PercentageRange), PercentageRangeValue);
			SetPercentageRange(PercentageRangeValue);
		}

		{
			double PercentageOffsetValue = PercentageOffset;
			ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, PercentageOffset), PercentageOffsetValue);
			SetPercentageOffset(PercentageOffsetValue);
		}

		{
			int64 CharacterRangeCountValue = CharacterRangeCount;
			ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, CharacterRangeCount), CharacterRangeCountValue);
			SetCharacterRangeCount(CharacterRangeCountValue);
		}

		{
			int64 CharacterOffsetCountValue = CharacterOffsetCount;
			ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, CharacterOffsetCount), CharacterOffsetCountValue);
			SetCharacterOffsetCount(CharacterOffsetCountValue);
		}

		{
			int64 WordRangeCountValue = WordRangeCount;
			ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, WordRangeCount), WordRangeCountValue);
			SetWordRangeCount(WordRangeCountValue);
		}

		{
			int64 WordOffsetCountValue = WordOffsetCount;
			ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, WordOffsetCount), WordOffsetCountValue);
			SetWordOffsetCount(WordOffsetCountValue);
		}

		{
			uint64 DirectionValue = static_cast<uint64>(Direction);
			ResolverArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, Direction), DirectionValue);
			SetDirection(static_cast<EPropertyAnimatorTextResolverRangeDirection>(DirectionValue));
		}

		return true;
	}

	return false;
}

bool UPropertyAnimatorTextResolver::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ResolverArchive = OutValue->AsMutableObject();

		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, Unit), static_cast<uint64>(Unit));
		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, PercentageRange), PercentageRange);
		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, PercentageOffset), PercentageOffset);
		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, CharacterRangeCount), static_cast<int64>(CharacterRangeCount));
		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, CharacterOffsetCount), static_cast<int64>(CharacterOffsetCount));
		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, WordRangeCount), static_cast<int64>(WordRangeCount));
		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, WordOffsetCount), static_cast<int64>(WordOffsetCount));
		ResolverArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorTextResolver, Direction), static_cast<uint64>(Direction));

		return true;
	}

	return false;
}
