// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/PropertyAnimatorCoreData.h"
#include "Properties/PropertyAnimatorCoreResolver.h"
#include "PropertyAnimatorTextResolver.generated.h"

UENUM()
enum class EPropertyAnimatorTextResolverRangeUnit : uint8
{
	Percentage,
	Character,
	Word
};

UENUM()
enum class EPropertyAnimatorTextResolverRangeDirection : uint8
{
	LeftToRight,
	RightToLeft,
	FromCenter
};

/**
 * Text characters properties resolver
 * Since each character in text are transient and regenerated on change
 * We need to have a resolver that will resolve each character in the text when needed
 * We manipulate a single property that underneath means we manipulate all text characters properties
 */
UCLASS()
class UPropertyAnimatorTextResolver : public UPropertyAnimatorCoreResolver
{
	GENERATED_BODY()

public:
	UPropertyAnimatorTextResolver()
		: UPropertyAnimatorCoreResolver(TEXT("TextChars"))
	{}

	PROPERTYANIMATOR_API void SetUnit(EPropertyAnimatorTextResolverRangeUnit InUnit);
	EPropertyAnimatorTextResolverRangeUnit GetUnit() const
	{
		return Unit;
	}

	float GetPercentageRange() const
	{
		return PercentageRange;
	}

	void SetPercentageRange(float InPercentageRange);

	float GetPercentageOffset() const
	{
		return PercentageOffset;
	}

	void SetPercentageOffset(float InPercentageOffset);

	int32 GetCharacterRangeCount() const
	{
		return CharacterRangeCount;
	}

	void SetCharacterRangeCount(int32 InCharacterRangeCount);

	int32 GetCharacterOffsetCount() const
	{
		return CharacterOffsetCount;
	}

	void SetCharacterOffsetCount(int32 InCharacterOffsetCount);

	int32 GetWordRangeCount() const
	{
		return WordRangeCount;
	}

	void SetWordRangeCount(int32 InWordRangeCount);

	int32 GetWordOffsetCount() const
	{
		return WordOffsetCount;
	}

	void SetWordOffsetCount(int32 InWordOffsetCount);
 
	PROPERTYANIMATOR_API void SetDirection(EPropertyAnimatorTextResolverRangeDirection InDirection);
	EPropertyAnimatorTextResolverRangeDirection GetDirection() const
	{
		return Direction;
	}

protected:
	//~ Begin UPropertyAnimatorCoreResolver
	virtual bool FixUpProperty(FPropertyAnimatorCoreData& InOldProperty) override;
	virtual void GetTemplateProperties(UObject* InContext, TSet<FPropertyAnimatorCoreData>& OutProperties, const TArray<FName>* InSearchPath) override;
	virtual void ResolveTemplateProperties(const FPropertyAnimatorCoreData& InTemplateProperty, TArray<FPropertyAnimatorCoreData>& OutProperties, bool bInForEvaluation) override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorCoreResolver

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	EPropertyAnimatorTextResolverRangeUnit Unit = EPropertyAnimatorTextResolverRangeUnit::Percentage;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Delta="1", ClampMin="0", ClampMax="100", Units=Percent, EditCondition="Unit == EPropertyAnimatorTextResolverRangeUnit::Percentage", EditConditionHides))
	float PercentageRange = 100.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Delta="1", Units=Percent, EditCondition="Unit == EPropertyAnimatorTextResolverRangeUnit::Percentage", EditConditionHides))
	float PercentageOffset = 0.f;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Delta="1", ClampMin="0", EditCondition="Unit == EPropertyAnimatorTextResolverRangeUnit::Character", EditConditionHides))
	int32 CharacterRangeCount = 100;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Delta="1", EditCondition="Unit == EPropertyAnimatorTextResolverRangeUnit::Character", EditConditionHides))
	int32 CharacterOffsetCount = 0;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Delta="1", ClampMin="0", EditCondition="Unit == EPropertyAnimatorTextResolverRangeUnit::Word", EditConditionHides))
	int32 WordRangeCount = 100;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Delta="1", EditCondition="Unit == EPropertyAnimatorTextResolverRangeUnit::Word", EditConditionHides))
	int32 WordOffsetCount = 0;

	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	EPropertyAnimatorTextResolverRangeDirection Direction = EPropertyAnimatorTextResolverRangeDirection::LeftToRight;
};