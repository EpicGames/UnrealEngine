// Copyright Epic Games, Inc. All Rights Reserved.

#include "Properties/PropertyAnimatorVectorContext.h"

#include "Animators/PropertyAnimatorCoreBase.h"

void UPropertyAnimatorVectorContext::SetAmplitudeMin(const FVector& InAmplitude)
{
	AmplitudeMin = GetClampedAmplitude(InAmplitude);
}

void UPropertyAnimatorVectorContext::SetAmplitudeMax(const FVector& InAmplitude)
{
	AmplitudeMax = GetClampedAmplitude(InAmplitude);
}

#if WITH_EDITOR
void UPropertyAnimatorVectorContext::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberPropertyName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorVectorContext, AmplitudeMin))
	{
		SetAmplitudeMin(AmplitudeMin);
	}
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorVectorContext, AmplitudeMax))
	{
		SetAmplitudeMax(AmplitudeMax);
	}
}
#endif

bool UPropertyAnimatorVectorContext::EvaluateProperty(const FPropertyAnimatorCoreData& InProperty, const FInstancedPropertyBag& InAnimatorResult, FInstancedPropertyBag& OutEvaluatedValues)
{
	const TValueOrError<float, EPropertyBagResult> AlphaResult = InAnimatorResult.GetValueFloat(UPropertyAnimatorCoreBase::AlphaParameterName);
	const TValueOrError<float, EPropertyBagResult> MagnitudeResult = InAnimatorResult.GetValueFloat(UPropertyAnimatorCoreBase::MagnitudeParameterName);

	if (AlphaResult.HasValue())
	{
		const FName PropertyHash = InProperty.GetLocatorPathHash();
		OutEvaluatedValues.AddProperty(PropertyHash, EPropertyBagPropertyType::Struct, TBaseStructure<FVector>::Get());
		OutEvaluatedValues.SetValueStruct(PropertyHash, MagnitudeResult.GetValue() * FMath::Lerp(AmplitudeMin, AmplitudeMax, AlphaResult.GetValue()));
		return true;
	}

	return false;
}

void UPropertyAnimatorVectorContext::OnAnimatedPropertyLinked()
{
	Super::OnAnimatedPropertyLinked();

	AmplitudeClampMin.Reset();
	AmplitudeClampMax.Reset();

#if WITH_EDITOR
	const FPropertyAnimatorCoreData& Property = GetAnimatedProperty();
	const FProperty* LeafProperty = Property.GetLeafProperty();

	checkf(LeafProperty, TEXT("Animated leaf property must be valid"))

	// Assign Min and Max value based on editor meta data available
	if (LeafProperty->IsA<FStructProperty>())
	{
		if (LeafProperty->HasMetaData(TEXT("ClampMin")))
		{
			AmplitudeMin = FVector(LeafProperty->GetFloatMetaData(FName("ClampMin")));
			AmplitudeClampMin = AmplitudeMin;
		}
		else if (LeafProperty->HasMetaData(TEXT("UIMin")))
		{
			AmplitudeMin = FVector(LeafProperty->GetFloatMetaData(FName("UIMin")));
		}

		if (LeafProperty->HasMetaData(TEXT("ClampMax")))
		{
			AmplitudeMax = FVector(LeafProperty->GetFloatMetaData(FName("ClampMax")));
			AmplitudeClampMax = AmplitudeMax;
		}
		else if (LeafProperty->HasMetaData(TEXT("UIMax")))
		{
			AmplitudeMax = FVector(LeafProperty->GetFloatMetaData(FName("UIMax")));
		}
	}
#endif
}

bool UPropertyAnimatorVectorContext::ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue)
{
	if (Super::ImportPreset(InPreset, InValue) && InValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ContextArchive = InValue->AsMutableObject();

		FString AmplitudeMinValue = AmplitudeMin.ToString();
		ContextArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorVectorContext, AmplitudeMin), AmplitudeMinValue);
		FVector ParseAmplitudeMin;
		ParseAmplitudeMin.InitFromString(AmplitudeMinValue);
		SetAmplitudeMin(ParseAmplitudeMin);

		FString AmplitudeMaxValue = AmplitudeMax.ToString();
		ContextArchive->Get(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorVectorContext, AmplitudeMax), AmplitudeMaxValue);
		FVector ParseAmplitudeMax;
		ParseAmplitudeMax.InitFromString(AmplitudeMaxValue);
		SetAmplitudeMax(ParseAmplitudeMax);

		return true;
	}

	return false;
}

bool UPropertyAnimatorVectorContext::ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const
{
	if (Super::ExportPreset(InPreset, OutValue) && OutValue->IsObject())
	{
		const TSharedPtr<FPropertyAnimatorCorePresetObjectArchive> ContextArchive = OutValue->AsMutableObject();

		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorVectorContext, AmplitudeMin), AmplitudeMin.ToString());
		ContextArchive->Set(GET_MEMBER_NAME_STRING_CHECKED(UPropertyAnimatorVectorContext, AmplitudeMax), AmplitudeMax.ToString());

		return true;
	}

	return false;
}

FVector UPropertyAnimatorVectorContext::GetClampedAmplitude(FVector InAmplitude)
{
	if (AmplitudeClampMin.IsSet())
	{
		const FVector& MinAmplitude = AmplitudeClampMin.GetValue();
		InAmplitude = InAmplitude.ComponentMax(MinAmplitude);
	}

	if (AmplitudeClampMax.IsSet())
	{
		const FVector& MaxAmplitude = AmplitudeClampMax.GetValue();
		InAmplitude = InAmplitude.ComponentMin(MaxAmplitude);
	}

	return InAmplitude;
}
