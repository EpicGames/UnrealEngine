// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorShared.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector2D.h"

namespace UE::PropertyAnimator
{

static TMap<EPropertyAnimatorEasingFunction, float(*)(float, EPropertyAnimatorEasingType)> GEasingFunctions =
	{
		{ EPropertyAnimatorEasingFunction::Linear, &Easing::Linear },
		{ EPropertyAnimatorEasingFunction::Sine, &Easing::Sine },
		{ EPropertyAnimatorEasingFunction::Quad, &Easing::Quad },
		{ EPropertyAnimatorEasingFunction::Cubic, &Easing::Cubic },
		{ EPropertyAnimatorEasingFunction::Quart, &Easing::Quart },
		{ EPropertyAnimatorEasingFunction::Quint, &Easing::Quint },
		{ EPropertyAnimatorEasingFunction::Expo, &Easing::Expo },
		{ EPropertyAnimatorEasingFunction::Circ, &Easing::Circ },
		{ EPropertyAnimatorEasingFunction::Back, &Easing::Back },
		{ EPropertyAnimatorEasingFunction::Elastic, &Easing::Elastic },
		{ EPropertyAnimatorEasingFunction::Bounce, &Easing::Bounce }
	};

float Easing::Ease(float InProgress, EPropertyAnimatorEasingFunction InFunction, EPropertyAnimatorEasingType InType)
{
	return GEasingFunctions[InFunction](InProgress, InType);
}

float Easing::Linear(float InProgress, EPropertyAnimatorEasingType InType)
{
	return InProgress;
}

float Easing::Sine(float InProgress, EPropertyAnimatorEasingType InType)
{
	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return 1 - FMath::Cos((InProgress * PI) / 2);

		case EPropertyAnimatorEasingType::Out:
			return FMath::Sin((InProgress * PI) / 2);

		case EPropertyAnimatorEasingType::InOut:
			return -(FMath::Cos(InProgress * PI) - 1) / 2;
	}
	return 0.f;
}

float Easing::Quad(float InProgress, EPropertyAnimatorEasingType InType)
{
	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return InProgress * InProgress;

		case EPropertyAnimatorEasingType::Out:
			return 1 - (1 - InProgress) * (1 - InProgress);

		case EPropertyAnimatorEasingType::InOut:
			return InProgress < 0.5 ? 2 * InProgress * InProgress : 1 - FMath::Pow(-2 * InProgress + 2, 2) / 2;
	}
	return 0.f;
}

float Easing::Cubic(float InProgress, EPropertyAnimatorEasingType InType)
{
	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return InProgress * InProgress * InProgress;

		case EPropertyAnimatorEasingType::Out:
			return 1 - FMath::Pow(1 - InProgress, 3);

		case EPropertyAnimatorEasingType::InOut:
			return InProgress < 0.5 ? 4 * InProgress * InProgress * InProgress : 1 - FMath::Pow(-2 * InProgress + 2, 3) / 2;
	}
	return 0.f;
}

float Easing::Quart(float InProgress, EPropertyAnimatorEasingType InType)
{
	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return InProgress * InProgress * InProgress * InProgress;

		case EPropertyAnimatorEasingType::Out:
			return 1 - FMath::Pow(1 - InProgress, 4);

		case EPropertyAnimatorEasingType::InOut:
			return InProgress < 0.5 ? 8 * InProgress * InProgress * InProgress * InProgress : 1 - FMath::Pow(-2 * InProgress + 2, 4) / 2;
	}
	return 0.f;
}

float Easing::Quint(float InProgress, EPropertyAnimatorEasingType InType)
{
	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return InProgress * InProgress * InProgress * InProgress * InProgress;

		case EPropertyAnimatorEasingType::Out:
			return 1 - FMath::Pow(1 - InProgress, 5);

		case EPropertyAnimatorEasingType::InOut:
			return InProgress < 0.5 ? 16 * InProgress * InProgress * InProgress * InProgress * InProgress : 1 - FMath::Pow(-2 * InProgress + 2, 5) / 2;
	}
	return 0.f;
}

float Easing::Expo(float InProgress, EPropertyAnimatorEasingType InType)
{
	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return InProgress == 0 ? 0 : FMath::Pow(2, 10 * InProgress - 10);

		case EPropertyAnimatorEasingType::Out:
			return InProgress == 1 ? 1 : 1 - FMath::Pow(2, -10 * InProgress);

		case EPropertyAnimatorEasingType::InOut:
			if (InProgress == 0)
			{
				return 0.f;
			}
			else if (InProgress == 1)
			{
				return 1.f;
			}
			else if (InProgress < 0.5)
			{
				return FMath::Pow(2, 20 * InProgress - 10) / 2;
			}
			else
			{
				return (2 - FMath::Pow(2, -20 * InProgress + 10)) / 2;
			}
	}
	return 0.f;
}

float Easing::Circ(float InProgress, EPropertyAnimatorEasingType InType)
{
	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return 1 - FMath::Sqrt(1 - FMath::Pow(InProgress, 2));

		case EPropertyAnimatorEasingType::Out:
			return FMath::Sqrt(1 - FMath::Pow(InProgress - 1, 2));

		case EPropertyAnimatorEasingType::InOut:
			return InProgress < 0.5
			  ? (1 - FMath::Sqrt(1 - FMath::Pow(2 * InProgress, 2))) / 2
			  : (FMath::Sqrt(1 - FMath::Pow(-2 * InProgress + 2, 2)) + 1) / 2;
	}
	return 0.f;
}

float Easing::Back(float InProgress, EPropertyAnimatorEasingType InType)
{
	constexpr float C1 = 1.70158f;
	constexpr float C2 = C1 * 1.525f;
	constexpr float C3 = C1 + 1.0f;

	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return C3 * InProgress * InProgress * InProgress - C1 * InProgress * InProgress;

		case EPropertyAnimatorEasingType::Out:
			return 1 + C3 * FMath::Pow(InProgress - 1, 3) + C1 * FMath::Pow(InProgress - 1, 2);

		case EPropertyAnimatorEasingType::InOut:
			return InProgress < 0.5
			  ? (FMath::Pow(2 * InProgress, 2) * ((C2 + 1) * 2 * InProgress - C2)) / 2
			  : (FMath::Pow(2 * InProgress - 2, 2) * ((C2 + 1) * (InProgress * 2 - 2) + C2) + 2) / 2;
	}
	return 0.f;
}

float Easing::Elastic(float InProgress, EPropertyAnimatorEasingType InType)
{
	constexpr float C4 = (2 * PI) / 3.0f;
	constexpr float C5 = (2 * PI) / 4.5f;

	if (InProgress == 0)
	{
		return 0.f;
	}
	else if (InProgress == 1)
	{
		return 1.f;
	}

	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return -FMath::Pow(2, 10 * InProgress - 10) * FMath::Sin((InProgress * 10 - 10.75) * C4);

		case EPropertyAnimatorEasingType::Out:
			return FMath::Pow(2, -10 * InProgress) * FMath::Sin((InProgress * 10 - 0.75) * C4) + 1;

		case EPropertyAnimatorEasingType::InOut:
			if (InProgress < 0.5)
			{
				return -(FMath::Pow(2, 20 * InProgress - 10) * FMath::Sin((20 * InProgress - 11.125) * C5)) / 2;
			}
			else
			{
				return (FMath::Pow(2, -20 * InProgress + 10) * FMath::Sin((20 * InProgress - 11.125) * C5)) / 2 + 1;
			}
	}
	return 0.f;
}

float Easing::Bounce(float InProgress, EPropertyAnimatorEasingType InType)
{
	constexpr float N1 = 7.5625f;
	constexpr float D1 = 2.75f;

	switch (InType)
	{
		case EPropertyAnimatorEasingType::In:
			return 1 - Bounce(1 - InProgress, EPropertyAnimatorEasingType::Out);

		case EPropertyAnimatorEasingType::Out:
			if (InProgress < 1 / D1)
			{
				return N1 * InProgress * InProgress;
			}
			else if (InProgress < 2 / D1)
			{
				InProgress -= 1.5f / D1;
				return N1 * InProgress * InProgress + 0.75f;
			}
			else if (InProgress < 2.5f / D1)
			{
				InProgress -= 2.25f / D1;
				return N1 * InProgress * InProgress + 0.9375f;
			}
			else
			{
				InProgress -= 2.625f / D1;
				return N1 * InProgress * InProgress + 0.984375f;
			}

		case EPropertyAnimatorEasingType::InOut:
			return InProgress < 0.5
			  ? (1 - Bounce(1 - 2 * InProgress, EPropertyAnimatorEasingType::Out)) / 2
			  : (1 + Bounce(2 * InProgress - 1, EPropertyAnimatorEasingType::Out)) / 2;
	}
	return 0.f;
}

static TMap<EPropertyAnimatorWaveFunction, double(*)(double, double, double, double)> GWaveFunctions =
	{
		{ EPropertyAnimatorWaveFunction::Sine, &Wave::Sine },
		{ EPropertyAnimatorWaveFunction::Cosine, &Wave::Cosine },
		{ EPropertyAnimatorWaveFunction::Square, &Wave::Square },
		{ EPropertyAnimatorWaveFunction::InvertedSquare, &Wave::InvertedSquare },
		{ EPropertyAnimatorWaveFunction::Sawtooth, &Wave::Sawtooth },
		{ EPropertyAnimatorWaveFunction::Triangle, &Wave::Triangle },
		{ EPropertyAnimatorWaveFunction::Bounce, &Wave::Bounce },
		{ EPropertyAnimatorWaveFunction::Pulse, &Wave::Pulse },
		{ EPropertyAnimatorWaveFunction::Perlin, &Wave::Perlin }
	};

double Wave::Wave(double InTime, double InAmplitude, double InFrequency, double InOffset, EPropertyAnimatorWaveFunction InFunction)
{
	return GWaveFunctions[InFunction](InTime, InAmplitude, InFrequency, InOffset);
}

double Wave::Sine(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	return InAmplitude * FMath::Sin(2 * PI * InFrequency * InTime + InOffset);
}

double Wave::Cosine(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	return InAmplitude * FMath::Cos(2 * PI * InFrequency * InTime + InOffset);
}

double Wave::Square(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	return InAmplitude * FMath::Sign(FMath::Sin(2 * PI * InFrequency * InTime + InOffset));
}

double Wave::InvertedSquare(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	return Square(InTime, -InAmplitude, InFrequency, InOffset);
}

double Wave::Sawtooth(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	return InAmplitude * (2 * FMath::Wrap(InTime + InOffset, 0.0, 1.0f / InFrequency) * InFrequency - 1);
}

double Wave::Triangle(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	const double Period = 1.0 / InFrequency;
	const double NormalizedTime = FMath::Wrap(InTime + InOffset, 0.0, Period) / Period;

	return (NormalizedTime >= 0.5)
		? 2 * FMath::GetMappedRangeValueClamped(FVector2D(0.5, 1), FVector2D(1, 0), NormalizedTime) - 1
		: 2 * FMath::GetMappedRangeValueClamped(FVector2D(0, 0.5), FVector2D(0, 1), NormalizedTime) - 1;
}

double Wave::Bounce(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	const double Period = 1.0 / InFrequency;
	const double NormalizedTime = FMath::Wrap(InTime + InOffset, 0.0, Period) / Period;

	return (NormalizedTime < 0.5)
		? (4.0 * InAmplitude * NormalizedTime - InAmplitude)
		: (-4.0 * InAmplitude * NormalizedTime + 3.0 * InAmplitude);
}

double Wave::Pulse(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	const double Period = 1.0 / InFrequency;
	constexpr double DutyCycle = 0.2; // 20% duty cycle
	const double Time = DutyCycle * Period;
	const double NormalizedTime = FMath::Wrap(InTime + InOffset, 0.0, Period);

	return (NormalizedTime < Time) ? InAmplitude : -InAmplitude;
}

double Wave::Perlin(double InTime, double InAmplitude, double InFrequency, double InOffset)
{
	return InAmplitude * static_cast<double>(FMath::PerlinNoise1D(InTime * InFrequency + InOffset));
}

}
