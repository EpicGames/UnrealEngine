// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/PropertyAnimatorSettings.h"

#if WITH_EDITOR
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#endif

UPropertyAnimatorSettings::UPropertyAnimatorSettings()
{
	CategoryName = TEXT("Motion Design");
	SectionName = TEXT("Property Animator");

	FPropertyAnimatorCounterFormat DefaultFormat;
	DefaultFormat.FormatName = TEXT("Default");
	DefaultFormat.bTruncate = false;
	DefaultFormat.bUseSign = false;
	DefaultFormat.DecimalCharacter = TEXT(".");
	DefaultFormat.PaddingCharacter = TEXT("0");
	DefaultFormat.GroupingCharacter = TEXT(",");
	DefaultFormat.RoundingMode = EPropertyAnimatorCounterRoundingMode::None;
	DefaultFormat.MaxDecimalCount = 3;
	DefaultFormat.MinIntegerCount = 7;
	DefaultFormat.GroupingSize = 3;

	CounterFormatPresets.Add(DefaultFormat);
}

TSet<FName> UPropertyAnimatorSettings::GetCounterFormatNames() const
{
	TSet<FName> FormatNames;
	Algo::Transform(CounterFormatPresets, FormatNames, [](const FPropertyAnimatorCounterFormat& InFormat)
	{
		return InFormat.FormatName;
	});
	return FormatNames;
}

const FPropertyAnimatorCounterFormat* UPropertyAnimatorSettings::GetCounterFormat(FName InName) const
{
	if (const FPropertyAnimatorCounterFormat* Format = CounterFormatPresets.Find(FPropertyAnimatorCounterFormat(InName)))
	{
		return Format;
	}

	return nullptr;
}

#if WITH_EDITOR
bool UPropertyAnimatorSettings::AddCounterFormat(const FPropertyAnimatorCounterFormat& InNewFormat, bool bInOverride, bool bInSaveConfig)
{
	if (InNewFormat.FormatName.IsNone())
	{
		return false;
	}

	if (bInOverride)
	{
		CounterFormatPresets.Remove(InNewFormat);
	}

	bool bAlreadyInSet = false;
	FPropertyAnimatorCounterFormat& NewFormat = CounterFormatPresets.FindOrAdd(InNewFormat, &bAlreadyInSet);

	NewFormat.EnsureCharactersLength();

	if (!bAlreadyInSet)
	{
		if (bInSaveConfig)
		{
			SaveConfig();
			TryUpdateDefaultConfigFile();
		}

		return true;
	}

	return false;
}

void UPropertyAnimatorSettings::OpenSettings() const
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>(TEXT("Settings")))
	{
		SettingsModule->ShowViewer(GetContainerName(), GetCategoryName(), GetSectionName());
	}
}

void UPropertyAnimatorSettings::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	const FName MemberName = InPropertyChangedEvent.GetMemberPropertyName();

	if (MemberName == GET_MEMBER_NAME_CHECKED(UPropertyAnimatorSettings, CounterFormatPresets)
		&& InPropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		OnCounterFormatsChanged();
	}
}
#endif

void UPropertyAnimatorSettings::OnCounterFormatsChanged()
{
	for (FPropertyAnimatorCounterFormat& Format : CounterFormatPresets)
	{
		Format.EnsureCharactersLength();
	}
}

