// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Animators/PropertyAnimatorCounter.h"
#include "PropertyAnimatorSettings.generated.h"

/** Settings for motion design PropertyAnimator plugin */
UCLASS(Config=Engine, meta=(DisplayName="Property Animator"))
class UPropertyAnimatorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UPropertyAnimatorSettings();

	TSet<FName> GetCounterFormatNames() const;
	const FPropertyAnimatorCounterFormat* GetCounterFormat(FName InName) const;

#if WITH_EDITOR
	bool AddCounterFormat(const FPropertyAnimatorCounterFormat& InNewFormat, bool bInOverride = true, bool bInSaveConfig = true);
	void OpenSettings() const;
#endif

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnCounterFormatsChanged();

	UPROPERTY(Config, EditAnywhere, Category="Animator", meta=(TitleProperty="FormatName"))
	TSet<FPropertyAnimatorCounterFormat> CounterFormatPresets;
};
