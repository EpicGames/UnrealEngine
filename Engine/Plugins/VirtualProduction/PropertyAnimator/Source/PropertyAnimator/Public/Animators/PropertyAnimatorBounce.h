// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorNumericBase.h"
#include "PropertyAnimatorBounce.generated.h"

/**
 * Applies an additive bounce movement with various options on supported float properties
 */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"), Hidden, meta=(Deprecated, DeprecationMessage="Use preset bounce animator based on curve instead"))
class UPropertyAnimatorBounce : public UPropertyAnimatorNumericBase
{
	GENERATED_BODY()

public:
	PROPERTYANIMATOR_API void SetInvertEffect(bool bInvert);
	bool GetInvertEffect() const
	{
		return bInvertEffect;
	}

protected:
	virtual void OnInvertEffect() {}

	//~ Begin UPropertyAnimatorFloatBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const override;
	//~ End UPropertyAnimatorFloatBase

	/** Invert the effect result */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter="SetInvertEffect", Getter="GetInvertEffect", Category="Animator")
	bool bInvertEffect = false;
};