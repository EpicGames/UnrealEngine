// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Timespan.h"
#include "PropertyAnimatorTextBase.h"
#include "PropertyAnimatorClock.generated.h"

/** Mode supported for properties value */
UENUM(BlueprintType)
enum class EPropertyAnimatorClockMode : uint8
{
	/** Local time of the machine */
	LocalTime,
	/** Specified duration elapsing until it reaches 0 */
	Countdown,
	/** Shows the current time elapsed */
	Stopwatch
};

/** Animate supported string properties to display time */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"))
class UPropertyAnimatorClock : public UPropertyAnimatorTextBase
{
	GENERATED_BODY()

public:
	static void RegisterFormat(TCHAR InChar, TFunction<FString(const FDateTime&)> InFormatter);
	static void UnregisterFormat(TCHAR InChar);
	static FString FormatDateTime(const FDateTime& InDateTime, const FString& InDisplayFormat, const FString& InMask);

	UPropertyAnimatorClock();

	PROPERTYANIMATOR_API void SetDisplayFormat(const FString& InDisplayFormat);
	const FString& GetDisplayFormat() const
	{
		return DisplayFormat;
	}

	PROPERTYANIMATOR_API void SetDisplayMask(const FString& InPadding);
	const FString& GetDisplayMask() const
	{
		return DisplayMask;
	}

protected:
	//~ Begin UPropertyAnimatorCoreBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual void EvaluateProperties(FInstancedPropertyBag& InParameters) override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorCoreBase

	/**
	 * Display date time format : 
	 * %a - Weekday, eg) Sun
	 * %A - Weekday, eg) Sunday
	 * %w - Weekday, 0-6 (Sunday is 0)
	 * %y - Year, YY
	 * %Y - Year, YYYY
	 * %b - Month, eg) Jan
	 * %B - Month, eg) January
	 * %m - Month, 01-12
	 * %n - Month, 1-12
	 * %d - Day, 01-31
	 * %e - Day, 1-31
	 * %j - Day of the Year, 001-366
	 * %J - Day of the Year, 1-366
	 * %l - 12h Hour, 1-12
	 * %I - 12h Hour, 01-12
	 * %H - 24h Hour, 00-23
	 * %h - 24h Hour, 0-23
	 * %o - Hour, 0-inf
	 * %M - Minute, 00-59
	 * %N - Minute, 0-59
	 * %i - Minute, 0-inf
	 * %S - Second, 00-60
	 * %s - Second, 0-60
	 * %c - Second, 0-inf
	 * %f - Millisecond, 000-999
	 * %F - Millisecond, 0-999
	 * %p - AM or PM
	 * %P - am or PM
	 * %t - Ticks since midnight, January 1, 0001
	 */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	FString DisplayFormat = TEXT("%H:%M:%S");

	/**
	 * Used as padding to complete the display format
	 * eg: DisplayMask=****** and DisplayFormat=%c would result in ***999
	 */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	FString DisplayMask = TEXT("");
};