// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorTextBase.h"
#include "StructUtils/InstancedStruct.h"
#include "PropertyAnimatorCounter.generated.h"

/** Enumerates all rounding mode available */
UENUM(BlueprintType)
enum class EPropertyAnimatorCounterRoundingMode : uint8
{
	None,
	Round,
	Floor,
	Ceil
};

/** Format options used to convert a number to string */
USTRUCT(BlueprintType)
struct FPropertyAnimatorCounterFormat
{
	GENERATED_BODY()

	FPropertyAnimatorCounterFormat() = default;

	explicit FPropertyAnimatorCounterFormat(FName InFormatName)
		: FormatName(InFormatName)
	{}

	/** Format friendly name */
	UPROPERTY(EditAnywhere, Category="Animator")
	FName FormatName;

	/** Minimum number of integer before the decimal separator for padding */
	UPROPERTY(EditAnywhere, Category="Animator")
	uint8 MinIntegerCount = 0;

	/** Maximum number of decimal precision after the decimal separator */
	UPROPERTY(EditAnywhere, Category="Animator")
	uint8 MaxDecimalCount = 3;

	/** Used to group numbers together like thousands */
	UPROPERTY(EditAnywhere, Category="Animator")
	uint8 GroupingSize = 3;

	/** Decimal separator character */
	UPROPERTY(EditAnywhere, Category="Animator")
	FString DecimalCharacter = TEXT(".");

	/** Thousands separator character */
	UPROPERTY(EditAnywhere, Category="Animator")
	FString GroupingCharacter = TEXT(",");

	/** Filling character for leading blanks */
	UPROPERTY(EditAnywhere, Category="Animator")
	FString PaddingCharacter = TEXT("0");

	/** Whether rounding the number is needed */
	UPROPERTY(EditAnywhere, Category="Animator")
	EPropertyAnimatorCounterRoundingMode RoundingMode = EPropertyAnimatorCounterRoundingMode::None;

	/** Add a prefix symbol to show the sign of the number (+, -) */
	UPROPERTY(EditAnywhere, Category="Animator")
	bool bUseSign = false;

	/** Truncate when the value exceeds the display format */
	UPROPERTY(EditAnywhere, Category="Animator")
	bool bTruncate = false;

	/** Clamp custom characters to one char only */
	void EnsureCharactersLength();

	/** Format a number using these options */
	FString FormatNumber(double InNumber) const;

	friend uint32 GetTypeHash(const FPropertyAnimatorCounterFormat& InItem)
	{
		return GetTypeHash(InItem.FormatName);
	}

	bool operator==(const FPropertyAnimatorCounterFormat& InOther) const
	{
		return FormatName.IsEqual(InOther.FormatName);
	}

	bool operator!=(const FPropertyAnimatorCounterFormat& InOther) const
	{
		return !FormatName.IsEqual(InOther.FormatName);
	}
};

/** Animate supported string properties to display a counter */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"))
class UPropertyAnimatorCounter : public UPropertyAnimatorTextBase
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	PROPERTYANIMATOR_API static FName GetUseCustomFormatPropertyName();
#endif

	UPropertyAnimatorCounter();

	void SetDisplayPattern(const FText& InPattern);
	const FText& GetDisplayPattern() const
	{
		return DisplayPattern;
	}

	void SetUseCustomFormat(bool bInUseCustom);
	bool GetUseCustomFormat() const
	{
		return bUseCustomFormat;
	}

	void SetPresetFormatName(FName InPresetName);
	FName GetPresetFormatName() const
	{
		return PresetFormatName;
	}

	void SetCustomFormat(const FPropertyAnimatorCounterFormat* InFormat);
	const FPropertyAnimatorCounterFormat* GetCustomFormat() const;

protected:
	FString FormatNumber(double InNumber) const;
	const FPropertyAnimatorCounterFormat* GetFormat() const;

	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UPropertyAnimatorCoreBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual void EvaluateProperties(FInstancedPropertyBag& InParameters) override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorCoreBase

	UFUNCTION()
	TArray<FName> GetAvailableFormatNames() const;

#if WITH_EDITOR
	UFUNCTION(CallInEditor)
	void OpenPropertyAnimatorSettings();

	UFUNCTION(CallInEditor)
	void SaveCustomFormatAsPreset();
#endif

	void OnUseCustomFormatChanged();
	void OnCustomFormatChanged();

	/** Display pattern for the output to add prefix and suffix, use {0} as replacement symbol */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	FText DisplayPattern = FText::FromString(TEXT("{0}"));

	/** Use available presets formats or custom format */
	UPROPERTY(EditInstanceOnly, Setter="SetUseCustomFormat", Getter="GetUseCustomFormat", Category="Animator")
	bool bUseCustomFormat = false;

	/** Preset format defined in the project settings */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(GetOptions="GetAvailableFormatNames", EditCondition="!bUseCustomFormat", EditConditionHides, HideEditConditionToggle))
	FName PresetFormatName;

	/** Custom format */
	UPROPERTY(EditInstanceOnly, Category="Animator", meta=(EditCondition="bUseCustomFormat", EditConditionHides, HideEditConditionToggle))
	TInstancedStruct<FPropertyAnimatorCounterFormat> CustomFormat;
};