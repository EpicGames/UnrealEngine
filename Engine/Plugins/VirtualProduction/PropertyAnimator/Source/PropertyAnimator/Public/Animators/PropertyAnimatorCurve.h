// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorNumericBase.h"
#include "PropertyAnimatorCurve.generated.h"

class UPropertyAnimatorEaseCurve;
class UPropertyAnimatorWaveCurve;

USTRUCT(BlueprintType)
struct FPropertyAnimatorCurveEasing
{
	GENERATED_BODY()

	UPROPERTY(EditInstanceOnly, DisplayName="Curve", Category="Animator", meta=(ForceShowEngineContent, ForceShowPluginContent))
	TObjectPtr<UPropertyAnimatorEaseCurve> EaseCurve;

	UPROPERTY(EditInstanceOnly, Interp, DisplayName="Duration", Category="Animator", meta=(ClampMin="0", Units=Seconds))
	float EaseDuration = 0.f;
};

/**
 * Applies a wave movement from a curve on supported float properties
 */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"))
class UPropertyAnimatorCurve : public UPropertyAnimatorNumericBase
{
	GENERATED_BODY()

public:
	UPropertyAnimatorCurve();

	void SetWaveCurve(UPropertyAnimatorWaveCurve* InCurve);

	UPropertyAnimatorWaveCurve* GetWaveCurve() const
	{
		return WaveCurve;
	}

	void SetEaseInEnabled(bool bInEnabled);

	bool GetEaseInEnabled() const
	{
		return bEaseInEnabled;
	}

	void SetEaseIn(const FPropertyAnimatorCurveEasing& InEasing);

	const FPropertyAnimatorCurveEasing& GetEaseIn() const
	{
		return EaseIn;
	}

	void SetEaseOutEnabled(bool bInEnabled);

	bool GetEaseOutEnabled() const
	{
		return bEaseOutEnabled;
	}

	void SetEaseOut(const FPropertyAnimatorCurveEasing& InEasing);

	const FPropertyAnimatorCurveEasing& GetEaseOut() const
	{
		return EaseOut;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UPropertyAnimatorFloatBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorFloatBase

	void OnEaseInChanged();
	void OnEaseOutChanged();
	virtual void OnCycleDurationChanged() override;

	/** Use ease in effect */
	UPROPERTY(EditInstanceOnly, Setter="SetEaseInEnabled", Getter="GetEaseInEnabled", DisplayName="InEnabled", Category="Animator", meta=(InlineEditConditionToggle))
	bool bEaseInEnabled = false;

	/** Ease in for this effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="In", Category="Animator", meta=(EditCondition="bEaseInEnabled"))
	FPropertyAnimatorCurveEasing EaseIn;

	/** The base curve to sample for the animation */
	UPROPERTY(EditInstanceOnly, Setter, Getter, DisplayName="Loop Curve", Category="Animator", meta=(ForceShowEngineContent, ForceShowPluginContent))
	TObjectPtr<UPropertyAnimatorWaveCurve> WaveCurve;

	/** Use ease out effect */
	UPROPERTY(EditInstanceOnly, Setter="SetEaseOutEnabled", Getter="GetEaseOutEnabled", DisplayName="OutEnabled", Category="Animator", meta=(InlineEditConditionToggle))
	bool bEaseOutEnabled = false;

	/** Ease out for this effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", DisplayName="Out", meta=(EditCondition="bEaseOutEnabled"))
    FPropertyAnimatorCurveEasing EaseOut;
};