// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorCoreBase.h"
#include "PropertyAnimatorNumericBase.generated.h"

UENUM(BlueprintType)
enum class EPropertyAnimatorCycleMode : uint8
{
	/** Disable cycle options */
	None	UMETA(Hidden),
	/** Cycle only once then stop */
	DoOnce,
	/** Cycle and repeat once we reached the end */
	Loop,
	/** Cycle and reverse repeat */
	PingPong
};

/**
 * Animate supported numeric properties with various options
 */
UCLASS(MinimalAPI, Abstract, AutoExpandCategories=("Animator"))
class UPropertyAnimatorNumericBase : public UPropertyAnimatorCoreBase
{
	GENERATED_BODY()

	friend class FPropertyAnimatorCoreEditorDetailCustomization;

public:
	PROPERTYANIMATOR_API void SetMagnitude(float InMagnitude);
	float GetMagnitude() const
	{
		return Magnitude;
	}

	PROPERTYANIMATOR_API void SetCycleMode(EPropertyAnimatorCycleMode InMode);
	EPropertyAnimatorCycleMode GetCycleMode() const
	{
		return CycleMode;
	}

	PROPERTYANIMATOR_API void SetCycleDuration(float InCycleDuration);
	float GetCycleDuration() const
	{
		return CycleDuration;
	}

	PROPERTYANIMATOR_API void SetCycleGapDuration(float InCycleGap);
	float GetCycleGapDuration() const
	{
		return CycleGapDuration;
	}

	PROPERTYANIMATOR_API void SetRandomTimeOffset(bool bInOffset);
	bool GetRandomTimeOffset() const
	{
		return bRandomTimeOffset;
	}

	PROPERTYANIMATOR_API void SetSeed(int32 InSeed);
	int32 GetSeed() const
	{
		return Seed;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	virtual void OnMagnitudeChanged() {}
	virtual void OnCycleDurationChanged() {}
	virtual void OnCycleModeChanged() {}
	virtual void OnSeedChanged() {}

	//~ Begin UPropertyAnimatorCoreBase
	virtual TSubclassOf<UPropertyAnimatorCoreContext> GetPropertyContextClass(const FPropertyAnimatorCoreData& InProperty) override;
	virtual EPropertyAnimatorPropertySupport IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const override;
	virtual void EvaluateProperties(FInstancedPropertyBag& InParameters) override;
	virtual void OnPropertyLinked(UPropertyAnimatorCoreContext* InLinkedProperty, EPropertyAnimatorPropertySupport InSupport) override;
	virtual bool IsTimeSourceSupported(UPropertyAnimatorCoreTimeSourceBase* InTimeSource) const override;
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorCoreBase

	/** Evaluate and return float value for a property */
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const
	{
		return false;
	}

	/** Magnitude for the effect on all properties */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(ClampMin="0"))
	float Magnitude = 1.f;

	/** Cycle mode for the effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(EditCondition="CycleMode != EPropertyAnimatorCycleMode::None", EditConditionHides))
	EPropertyAnimatorCycleMode CycleMode = EPropertyAnimatorCycleMode::Loop;

	/** Duration of one cycle for the effect = period of the effect */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(ClampMin="0", Units=Seconds, EditCondition="CycleMode != EPropertyAnimatorCycleMode::None", EditConditionHides))
	float CycleDuration = 1.f;

	/** Time gap between each cycle */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(ClampMin="0", Units=Seconds, EditCondition="CycleMode != EPropertyAnimatorCycleMode::DoOnce && CycleMode != EPropertyAnimatorCycleMode::None", EditConditionHides))
	float CycleGapDuration = 0.f;

	/** Use random time offset to add variation in animation */
	UPROPERTY(EditInstanceOnly, Setter="SetRandomTimeOffset", Getter="GetRandomTimeOffset", Category="Animator", meta=(InlineEditConditionToggle))
	bool bRandomTimeOffset = false;

	/** Seed to generate per property time offset */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(Seed, EditCondition="bRandomTimeOffset"))
	int32 Seed = 0;

private:
	/** Random stream for time offset */
	FRandomStream RandomStream = FRandomStream(Seed);
};