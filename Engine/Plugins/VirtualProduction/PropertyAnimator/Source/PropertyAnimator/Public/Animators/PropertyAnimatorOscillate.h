// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorNumericBase.h"
#include "PropertyAnimatorOscillate.generated.h"

UENUM(BlueprintType)
enum class EPropertyAnimatorOscillateFunction : uint8
{
	Sine,
	Cosine,
	Square,
	InvertedSquare,
	Sawtooth,
	Triangle,
};

/**
 * Applies an additive regular oscillate movement with various options on supported float properties
 */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"), Hidden, meta=(Deprecated, DeprecationMessage="Use preset oscillate animator based on curve instead"))
class UPropertyAnimatorOscillate : public UPropertyAnimatorNumericBase
{
	GENERATED_BODY()

public:
	PROPERTYANIMATOR_API void SetOscillateFunction(EPropertyAnimatorOscillateFunction InFunction);
	EPropertyAnimatorOscillateFunction GetOscillateFunction() const
	{
		return OscillateFunction;
	}

protected:
	//~ Begin UPropertyAnimatorFloatBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const override;
	//~ End UPropertyAnimatorFloatBase

	/** The oscillate function to feed current time elapsed */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Animator")
	EPropertyAnimatorOscillateFunction OscillateFunction = EPropertyAnimatorOscillateFunction::Sine;
};