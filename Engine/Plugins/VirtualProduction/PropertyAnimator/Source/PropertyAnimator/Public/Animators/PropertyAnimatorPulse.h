// Copyright Epic Games, Inc. All Rights Reserved.

/** The wave function to feed current time elapsed */

#pragma once

#include "Animators/PropertyAnimatorNumericBase.h"
#include "PropertyAnimatorShared.h"
#include "PropertyAnimatorPulse.generated.h"

/**
 * Applies an additive pulse movement with various options on supported float properties
 */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"), Hidden, meta=(Deprecated, DeprecationMessage="Use preset pulse animator based on curve instead"))
class UPropertyAnimatorPulse : public UPropertyAnimatorNumericBase
{
	GENERATED_BODY()

public:
	PROPERTYANIMATOR_API void SetEasingFunction(EPropertyAnimatorEasingFunction InEasingFunction);
	EPropertyAnimatorEasingFunction GetEasingFunction() const
	{
		return EasingFunction;
	}

	PROPERTYANIMATOR_API void SetEasingType(EPropertyAnimatorEasingType InEasingType);
	EPropertyAnimatorEasingType GetEasingType() const
	{
		return EasingType;
	}

protected:
	//~ Begin UPropertyAnimatorFloatBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const override;
	//~ End UPropertyAnimatorFloatBase

	/** The easing function to use to modify the base effect */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Animator")
	EPropertyAnimatorEasingFunction EasingFunction = EPropertyAnimatorEasingFunction::Linear;

	/** The type of effect for easing function */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Setter, Getter, Category="Animator", meta=(EditCondition="EasingFunction != EPropertyAnimatorEasingFunction::Line", EditConditionHides))
	EPropertyAnimatorEasingType EasingType = EPropertyAnimatorEasingType::InOut;
};