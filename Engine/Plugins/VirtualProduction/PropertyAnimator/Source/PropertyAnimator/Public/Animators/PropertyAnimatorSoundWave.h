// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorNumericBase.h"
#include "PropertyAnimatorSoundWave.generated.h"

class ULoudnessNRT;
class USoundWave;

/**
 * Applies a sampled sound wave movement with various options on supported float properties
 */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"))
class UPropertyAnimatorSoundWave : public UPropertyAnimatorNumericBase
{
	GENERATED_BODY()

public:
	UPropertyAnimatorSoundWave();

	PROPERTYANIMATOR_API void SetSampledSoundWave(USoundWave* InSoundWave);
	USoundWave* GetSampledSoundWave() const
	{
		return SampledSoundWave;
	}

	PROPERTYANIMATOR_API void SetLoop(bool bInLoop);
	bool GetLoop() const
	{
		return bLoop;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	//~ End UObject

	void OnSampledSoundWaveChanged();

	//~ Begin UPropertyAnimatorFloatBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorFloatBase

	/**
	 * The sound wave to analyse
	 * Cannot be switched at runtime, only in editor due to analyzer
	 * Analyzed audio will work at runtime since it is cached
	 */
	UPROPERTY(EditInstanceOnly, BlueprintReadOnly, Setter, Getter, Category="Animator")
	TObjectPtr<USoundWave> SampledSoundWave;

	/** Whether we keep looping after the duration has been reached or before 0 */
	UPROPERTY(EditInstanceOnly, BlueprintReadWrite, Setter="SetLoop", Getter="GetLoop", Category="Animator")
	bool bLoop = true;

private:
	/** Non-Real-Time audio analyser for loudness */
	UPROPERTY()
	TObjectPtr<ULoudnessNRT> AudioAnalyzer;
};