// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorCoreBase.h"
#include "PropertyAnimatorTextBase.generated.h"

/**
 * Animate supported text properties with various options
 */
UCLASS(MinimalAPI, Abstract, AutoExpandCategories=("Animator"))
class UPropertyAnimatorTextBase : public UPropertyAnimatorCoreBase
{
	GENERATED_BODY()

protected:
	//~ Begin UPropertyAnimatorCoreBase
	virtual EPropertyAnimatorPropertySupport IsPropertySupported(const FPropertyAnimatorCoreData& InPropertyData) const override;
	virtual void OnPropertyLinked(UPropertyAnimatorCoreContext* InLinkedProperty, EPropertyAnimatorPropertySupport InSupport) override;
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	//~ End UPropertyAnimatorCoreBase
};