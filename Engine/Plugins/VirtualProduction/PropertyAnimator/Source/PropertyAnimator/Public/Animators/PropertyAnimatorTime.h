// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorNumericBase.h"
#include "PropertyAnimatorTime.generated.h"

/**
 * Applies an additive time movement with various options on supported float properties
 */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"))
class UPropertyAnimatorTime : public UPropertyAnimatorNumericBase
{
	GENERATED_BODY()

protected:
	//~ Begin UPropertyAnimatorFloatBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const override;
	//~ End UPropertyAnimatorFloatBase
};