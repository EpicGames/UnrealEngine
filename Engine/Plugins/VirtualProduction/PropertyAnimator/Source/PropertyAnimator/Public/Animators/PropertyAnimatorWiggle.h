// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Animators/PropertyAnimatorNumericBase.h"
#include "PropertyAnimatorWiggle.generated.h"

/**
 * Applies a random wiggle movement with various options on supported numeric properties
 */
UCLASS(MinimalAPI, AutoExpandCategories=("Animator"))
class UPropertyAnimatorWiggle : public UPropertyAnimatorNumericBase
{
	GENERATED_BODY()

public:
	UPropertyAnimatorWiggle();

	PROPERTYANIMATOR_API void SetFrequency(float InFrequency);
	float GetFrequency() const
	{
		return Frequency;
	}

protected:
	//~ Begin UPropertyAnimatorFloatBase
	virtual void OnAnimatorRegistered(FPropertyAnimatorCoreMetadata& InMetadata) override;
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InPropertyData, UPropertyAnimatorCoreContext* InContext, FInstancedPropertyBag& InParameters, FInstancedPropertyBag& OutEvaluationResult) const override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorFloatBase

	/** Frequency for the effect, higher values will give you faster movements */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(ClampMin="0", Units=Hz))
	float Frequency = 1.f;
};