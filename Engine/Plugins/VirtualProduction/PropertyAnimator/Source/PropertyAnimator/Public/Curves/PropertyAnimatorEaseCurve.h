// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "PropertyAnimatorEaseCurve.generated.h"

/** Base class for property animator easing curves (0-1) */
UCLASS(MinimalAPI)
class UPropertyAnimatorEaseCurve : public UCurveFloat
{
	GENERATED_BODY()
};
