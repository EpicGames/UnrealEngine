// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/CurveFloat.h"
#include "PropertyAnimatorWaveCurve.generated.h"

/** Base class for property animator wave curves (inf-inf) */
UCLASS(MinimalAPI)
class UPropertyAnimatorWaveCurve : public UCurveFloat
{
	GENERATED_BODY()
};
