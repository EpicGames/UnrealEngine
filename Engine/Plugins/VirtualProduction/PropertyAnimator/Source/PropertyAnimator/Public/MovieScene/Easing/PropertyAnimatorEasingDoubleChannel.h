// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Misc/FrameTime.h"
#include "PropertyAnimatorEasingParameters.h"
#include "PropertyAnimatorEasingDoubleChannel.generated.h"

class UMovieSceneSection;

USTRUCT()
struct PROPERTYANIMATOR_API FPropertyAnimatorEasingDoubleChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	using CurveValueType = double;

	FPropertyAnimatorEasingDoubleChannel() = default;

	double Evaluate(double InBaseSeconds, double InSeconds) const;

	bool Evaluate(const UMovieSceneSection* InSection, FFrameTime InTime, double& OutValue) const;

	static double Evaluate(const FPropertyAnimatorEasingParameters& InParameters, double InBaseSeconds, double InSeconds);

	UPROPERTY(EditAnywhere, Category = "Easing")
	FPropertyAnimatorEasingParameters Parameters;
};

template<>
struct TMovieSceneChannelTraits<FPropertyAnimatorEasingDoubleChannel> : TMovieSceneChannelTraitsBase<FPropertyAnimatorEasingDoubleChannel>
{
#if WITH_EDITOR
	/** Easing Channels can have external values (they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<double> ExtendedEditorDataType;
#endif
};

inline bool EvaluateChannel(const UMovieSceneSection* InSection, const FPropertyAnimatorEasingDoubleChannel* InChannel, FFrameTime InTime, double& OutValue)
{
	return InChannel->Evaluate(InSection, InTime, OutValue);
}
