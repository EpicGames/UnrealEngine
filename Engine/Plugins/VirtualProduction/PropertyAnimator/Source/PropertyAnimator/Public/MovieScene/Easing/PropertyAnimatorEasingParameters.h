// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorShared.h"
#include "PropertyAnimatorEasingParameters.generated.h"

USTRUCT()
struct FPropertyAnimatorEasingParameters
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Easing")
	double Amplitude = 1.0;

	/** Length of the easing in seconds */
	UPROPERTY(EditAnywhere, Category="Easing")
	double EasingTime = 1.0;

	UPROPERTY(EditAnywhere, Category="Easing")
	double OffsetX = 0.0;

	UPROPERTY(EditAnywhere, Category="Easing")
	double OffsetY = 0.0;

	UPROPERTY(EditAnywhere, Category="Easing")
	EPropertyAnimatorEasingType EasingType = EPropertyAnimatorEasingType::In;

	UPROPERTY(EditAnywhere, Category="Easing")
	EPropertyAnimatorEasingFunction EasingFunction  = EPropertyAnimatorEasingFunction::Sine;
};
