// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Misc/FrameTime.h"
#include "PropertyAnimatorWaveParameters.h"
#include "PropertyAnimatorWaveDoubleChannel.generated.h"

class UMovieSceneSection;

USTRUCT()
struct PROPERTYANIMATOR_API FPropertyAnimatorWaveDoubleChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	using CurveValueType = double;

	FPropertyAnimatorWaveDoubleChannel() = default;

	double Evaluate(double InBaseSeconds, double InSeconds) const;

	bool Evaluate(const UMovieSceneSection* InSection, FFrameTime InTime, double& OutValue) const;

	static double Evaluate(const FPropertyAnimatorWaveParameters& InParameters, double InSeconds);

	UPROPERTY(EditAnywhere, Category = "Wave")
	FPropertyAnimatorWaveParameters Parameters;
};

template<>
struct TMovieSceneChannelTraits<FPropertyAnimatorWaveDoubleChannel> : TMovieSceneChannelTraitsBase<FPropertyAnimatorWaveDoubleChannel>
{
#if WITH_EDITOR
	/** Wave Channels can have external values (they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<double> ExtendedEditorDataType;
#endif
};

inline bool EvaluateChannel(const UMovieSceneSection* InSection, const FPropertyAnimatorWaveDoubleChannel* InChannel, FFrameTime InTime, double& OutValue)
{
	return InChannel->Evaluate(InSection, InTime, OutValue);
}
