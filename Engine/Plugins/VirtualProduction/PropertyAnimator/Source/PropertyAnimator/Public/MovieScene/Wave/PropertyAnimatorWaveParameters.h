// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyAnimatorShared.h"
#include "PropertyAnimatorWaveParameters.generated.h"

USTRUCT()
struct FPropertyAnimatorWaveParameters
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Wave")
	double Amplitude = 1.0;

	UPROPERTY(EditAnywhere, Category="Wave")
	double Frequency = 1.0;

	UPROPERTY(EditAnywhere, Category="Wave")
	double OffsetX = 0.0;

	UPROPERTY(EditAnywhere, Category="Wave")
	double OffsetY = 0.0;

	UPROPERTY(EditAnywhere, Category="Wave")
	EPropertyAnimatorWaveFunction WaveFunction  = EPropertyAnimatorWaveFunction::Sine;
};
