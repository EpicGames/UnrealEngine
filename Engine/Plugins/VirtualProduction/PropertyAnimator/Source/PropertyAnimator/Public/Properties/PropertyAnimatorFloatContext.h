// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/PropertyAnimatorCoreContext.h"
#include "PropertyAnimatorFloatContext.generated.h"

/** Property context used by animator for float/double properties */
UCLASS(MinimalAPI, BlueprintType)
class UPropertyAnimatorFloatContext : public UPropertyAnimatorCoreContext
{
	GENERATED_BODY()

public:
	PROPERTYANIMATOR_API void SetAmplitudeMin(double InAmplitude);
	double GetAmplitudeMin() const
	{
		return AmplitudeMin;
	}

	PROPERTYANIMATOR_API void SetAmplitudeMax(double InAmplitude);
	double GetAmplitudeMax() const
	{
		return AmplitudeMax;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UPropertyAnimatorCoreContext
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InProperty, const FInstancedPropertyBag& InAnimatorResult, FInstancedPropertyBag& OutEvaluatedValues) override;
	virtual void OnAnimatedPropertyLinked() override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorCoreContext

	double GetClampedAmplitude(double InAmplitude);

	/** The minimum value should be remapped to that values */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	double AmplitudeMin = -1.f;

	/** Some properties are clamped and cannot go past a specific min value */
	UPROPERTY()
	TOptional<double> AmplitudeClampMin;

	/** The maximum value should be remapped to that values */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator")
	double AmplitudeMax = 1.f;

	/** Some properties are clamped and cannot go past a specific min value */
	UPROPERTY()
	TOptional<double> AmplitudeClampMax;
};
