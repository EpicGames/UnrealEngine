// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Properties/PropertyAnimatorCoreContext.h"
#include "PropertyAnimatorVectorContext.generated.h"

/** Property context used by animator for vector properties */
UCLASS(MinimalAPI, BlueprintType)
class UPropertyAnimatorVectorContext : public UPropertyAnimatorCoreContext
{
	GENERATED_BODY()

public:
	PROPERTYANIMATOR_API void SetAmplitudeMin(const FVector& InAmplitude);
	const FVector& GetAmplitudeMin() const
	{
		return AmplitudeMin;
	}

	PROPERTYANIMATOR_API void SetAmplitudeMax(const FVector& InAmplitude);
	const FVector& GetAmplitudeMax() const
	{
		return AmplitudeMax;
	}

protected:
	//~ Begin UObject
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif
	//~ End UObject

	//~ Begin UPropertyAnimatorCoreContext
	virtual bool EvaluateProperty(const FPropertyAnimatorCoreData& InProperty, const FInstancedPropertyBag& InAnimatorResult, FInstancedPropertyBag& OutEvaluatedValues) override;
	virtual void OnAnimatedPropertyLinked() override;
	virtual bool ImportPreset(const UPropertyAnimatorCorePresetBase* InPreset, const TSharedRef<FPropertyAnimatorCorePresetArchive>& InValue) override;
	virtual bool ExportPreset(const UPropertyAnimatorCorePresetBase* InPreset, TSharedPtr<FPropertyAnimatorCorePresetArchive>& OutValue) const override;
	//~ End UPropertyAnimatorCoreContext

	FVector GetClampedAmplitude(FVector InAmplitude);

	/** The minimum value should be remapped to that values */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(AllowPreserveRatio))
	FVector AmplitudeMin = FVector::ZeroVector;

	/** Some properties are clamped and cannot go past a specific min value */
	UPROPERTY()
	TOptional<FVector> AmplitudeClampMin;

	/** The maximum value should be remapped to that values */
	UPROPERTY(EditInstanceOnly, Setter, Getter, Category="Animator", meta=(AllowPreserveRatio))
	FVector AmplitudeMax = FVector::ZeroVector;

	/** Some properties are clamped and cannot go past a specific min value */
	UPROPERTY()
	TOptional<FVector> AmplitudeClampMax;
};
