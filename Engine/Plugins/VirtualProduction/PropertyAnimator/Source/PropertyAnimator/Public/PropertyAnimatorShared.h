// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "PropertyAnimatorShared.generated.h"

UENUM(BlueprintType)
enum class EPropertyAnimatorEasingType : uint8
{
	In,
	Out,
	InOut
};

UENUM(BlueprintType)
enum class EPropertyAnimatorEasingFunction : uint8
{
	Linear,
	Sine,
	Quad,
	Cubic,
	Quart,
	Quint,
	Expo,
	Circ,
	Back,
	Elastic,
	Bounce
};

UENUM(BlueprintType)
enum class EPropertyAnimatorWaveFunction : uint8
{
	Sine,
	Cosine,
	Square,
	InvertedSquare,
	Sawtooth,
	Triangle,
	Bounce,
	Pulse,
	Perlin
};

namespace UE::PropertyAnimator
{
	/** Easings : InProgress needs to be normalized between 0 and 1, 0 meaning animation start and 1 meaning animation end */
	namespace Easing
	{
		float Ease(float InProgress, EPropertyAnimatorEasingFunction InFunction, EPropertyAnimatorEasingType InType);

		float Linear(float InProgress, EPropertyAnimatorEasingType InType);
		float Sine(float InProgress, EPropertyAnimatorEasingType InType);
		float Quad(float InProgress, EPropertyAnimatorEasingType InType);
		float Cubic(float InProgress, EPropertyAnimatorEasingType InType);
		float Quart(float InProgress, EPropertyAnimatorEasingType InType);
		float Quint(float InProgress, EPropertyAnimatorEasingType InType);
		float Expo(float InProgress, EPropertyAnimatorEasingType InType);
		float Circ(float InProgress, EPropertyAnimatorEasingType InType);
		float Back(float InProgress, EPropertyAnimatorEasingType InType);
		float Elastic(float InProgress, EPropertyAnimatorEasingType InType);
		float Bounce(float InProgress, EPropertyAnimatorEasingType InType);
	}

	/** Waves : return value should be between [-Amplitude, Amplitude] */
	namespace Wave
	{
		double Wave(double InTime, double InAmplitude, double InFrequency, double InOffset, EPropertyAnimatorWaveFunction InFunction);

		double Sine(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double Cosine(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double Square(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double InvertedSquare(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double Sawtooth(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double Triangle(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double Bounce(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double Pulse(double InTime, double InAmplitude, double InFrequency, double InOffset);
		double Perlin(double InTime, double InAmplitude, double InFrequency, double InOffset);
	}
}
