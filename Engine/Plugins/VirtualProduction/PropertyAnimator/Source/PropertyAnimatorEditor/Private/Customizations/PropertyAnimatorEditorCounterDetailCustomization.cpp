// Copyright Epic Games, Inc. All Rights Reserved.

#include "Customizations/PropertyAnimatorEditorCounterDetailCustomization.h"

#include "Animators/PropertyAnimatorCounter.h"
#include "DetailLayoutBuilder.h"

void FPropertyAnimatorEditorCounterDetailCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	UseCustomFormatHandle = InDetailBuilder.GetProperty(
		UPropertyAnimatorCounter::GetUseCustomFormatPropertyName(),
		UPropertyAnimatorCounter::StaticClass()
	);

	if (!UseCustomFormatHandle->IsValidHandle())
	{
		return;
	}

	// Due to instanced struct EditCondition not working and needing a refresh
	UseCustomFormatHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FPropertyAnimatorEditorCounterDetailCustomization::OnConditionPropertyChanged));
}

void FPropertyAnimatorEditorCounterDetailCustomization::OnConditionPropertyChanged() const
{
	if (UseCustomFormatHandle.IsValid() && UseCustomFormatHandle->IsValidHandle())
	{
		if (const TSharedPtr<IPropertyHandle> ParentPropertyHandle = UseCustomFormatHandle->GetParentHandle())
		{
			ParentPropertyHandle->RequestRebuildChildren();
		}
	}
}
