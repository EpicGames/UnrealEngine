// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

/** Details customization for UPropertyAnimatorCounter */
class FPropertyAnimatorEditorCounterDetailCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance()
	{
		return MakeShared<FPropertyAnimatorEditorCounterDetailCustomization>();
	}

	//~ Begin IDetailCustomization
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
	//~ End IDetailCustomization

protected:
	void OnConditionPropertyChanged() const;

	TSharedPtr<IPropertyHandle> UseCustomFormatHandle;
};
