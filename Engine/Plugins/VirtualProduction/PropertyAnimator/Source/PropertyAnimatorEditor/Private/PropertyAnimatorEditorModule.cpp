// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorEditorModule.h"

#include "Animators/PropertyAnimatorCounter.h"
#include "Customizations/PropertyAnimatorEditorCounterDetailCustomization.h"
#include "Modules/ModuleManager.h"
#include "MovieScene/Easing/PropertyAnimatorEasingDoubleChannel.h"
#include "MovieScene/Wave/PropertyAnimatorWaveDoubleChannel.h"
#include "PropertyEditorModule.h"
#include "Styles/PropertyAnimatorEditorStyle.h"

void FPropertyAnimatorEditorModule::StartupModule()
{
	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");
	RegisterCurveChannelInterface<FPropertyAnimatorWaveDoubleChannel>(SequencerModule);
	RegisterCurveChannelInterface<FPropertyAnimatorEasingDoubleChannel>(SequencerModule);

	// Init once
	FPropertyAnimatorEditorStyle::Get();

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout(UPropertyAnimatorCounter::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FPropertyAnimatorEditorCounterDetailCustomization::MakeInstance));

}

void FPropertyAnimatorEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor") && UObjectInitialized())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout(UPropertyAnimatorCounter::StaticClass()->GetFName());
	}
}

IMPLEMENT_MODULE(FPropertyAnimatorEditorModule, PropertyAnimatorEditor)
