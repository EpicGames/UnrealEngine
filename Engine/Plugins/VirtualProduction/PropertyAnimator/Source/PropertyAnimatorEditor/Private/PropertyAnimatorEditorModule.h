// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerModule.h"
#include "Modules/ModuleManager.h"
#include "Sequencer/PropertyAnimatorEditorCurveChannelInterface.h"
#include "Sequencer/PropertyAnimatorEditorCurveSectionMenuExtension.h"

class ISequencerModule;

class FPropertyAnimatorEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	template<typename InCurveChannelType>
	void RegisterCurveChannelInterface(ISequencerModule& InSequencerModule)
	{
		using FMenuExtensionType = TPropertyAnimatorEditorCurveSectionMenuExtension<InCurveChannelType>;
		using FChannelInterfaceType = TPropertyAnimatorEditorCurveChannelInterface<InCurveChannelType, FMenuExtensionType>;
		InSequencerModule.RegisterChannelInterface<InCurveChannelType>(MakeUnique<FChannelInterfaceType>());
	}
};
