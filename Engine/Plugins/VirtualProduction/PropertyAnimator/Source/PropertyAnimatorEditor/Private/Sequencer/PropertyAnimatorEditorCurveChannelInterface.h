// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/PerlinNoiseChannelInterface.h"
#include "CurveModel.h"
#include "ISequencerChannelInterface.h"
#include "MovieScene/PropertyAnimatorMovieSceneUtils.h"
#include "PropertyAnimatorEditorCurveSectionMenuExtension.h"
#include "SequencerChannelTraits.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SNullWidget.h"

template<typename InChannelType, typename InMenuExtensionType>
struct TPropertyAnimatorEditorCurveChannelInterface : ISequencerChannelInterface
{
	using FChannelType       = InChannelType;
	using FMenuExtensionType = InMenuExtensionType;
	using FCurveValueType    = typename InChannelType::CurveValueType;
	using FKeyEditor         = TSequencerKeyEditor<FChannelType, FCurveValueType>;
	using SKeyEditorWidget   = SNumericTextBlockKeyEditor<FChannelType, FCurveValueType>;

	//~ Begin ISequencerChannelInterface
	virtual FKeyHandle AddOrUpdateKey_Raw(FMovieSceneChannel*, UMovieSceneSection*, const void*, FFrameNumber, ISequencer&, const FGuid&, FTrackInstancePropertyBindings*) const override { return FKeyHandle::Invalid(); }
	virtual void DeleteKeys_Raw(FMovieSceneChannel*, TConstArrayView<FKeyHandle>, FFrameNumber) const override {}
	virtual void CopyKeys_Raw(FMovieSceneChannel*, const UMovieSceneSection*, FName, FMovieSceneClipboardBuilder&, TConstArrayView<FKeyHandle>) const override {}
	virtual void PasteKeys_Raw(FMovieSceneChannel*, UMovieSceneSection*, const FMovieSceneClipboardKeyTrack&, const FMovieSceneClipboardEnvironment&, const FSequencerPasteEnvironment&, TArray<FKeyHandle>&) const override {}
	virtual TSharedPtr<FStructOnScope> GetKeyStruct_Raw(FMovieSceneChannelHandle, FKeyHandle) const override { return nullptr; }
	virtual bool CanCreateKeyEditor_Raw(const FMovieSceneChannel*) const override { return true; }
	virtual void ExtendKeyMenu_Raw(FMenuBuilder&, TSharedPtr<FExtender>, TConstArrayView<FExtendKeyMenuParams>, TWeakPtr<ISequencer>) const override {}
	virtual void DrawKeys_Raw(FMovieSceneChannel*, TConstArrayView<FKeyHandle>, const UMovieSceneSection*, TArrayView<FKeyDrawParams>) const override {}
	virtual bool ShouldShowCurve_Raw(const FMovieSceneChannel*, UMovieSceneSection*) const override { return true; }
	virtual bool SupportsCurveEditorModels_Raw(const FMovieSceneChannelHandle&) const override { return false; }
	virtual TUniquePtr<FCurveModel> CreateCurveEditorModel_Raw(const FMovieSceneChannelHandle&, const UE::Sequencer::FCreateCurveEditorModelParams& Params) const override { return nullptr; }
	virtual TSharedPtr<UE::Sequencer::FChannelModel> CreateChannelModel_Raw(const FMovieSceneChannelHandle&, const UE::Sequencer::FSectionModel& InSection, FName) const override { return nullptr; }
	virtual TSharedPtr<UE::Sequencer::STrackAreaLaneView> CreateChannelView_Raw(const FMovieSceneChannelHandle&, TWeakPtr<UE::Sequencer::FChannelModel>, const UE::Sequencer::FCreateTrackLaneViewParams&) const override { return nullptr; }
	virtual void ExtendSectionMenu_Raw(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> InMenuExtender, TConstArrayView<FMovieSceneChannelHandle> InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer) const override;
	virtual TSharedPtr<ISidebarChannelExtension> ExtendSidebarMenu_Raw(FMenuBuilder& MenuBuilder, TSharedPtr<FExtender> InMenuExtender, TConstArrayView<FMovieSceneChannelHandle> InChannels, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections, TWeakPtr<ISequencer> InWeakSequencer) const override;
	virtual int32 DrawExtra_Raw(FMovieSceneChannel* InChannel, const UMovieSceneSection* InOwner, const FSequencerChannelPaintArgs& InPaintArgs, int32 InLayerId) const override;
	virtual TSharedRef<SWidget> CreateKeyEditor_Raw(const FMovieSceneChannelHandle& InChannel, const UE::Sequencer::FCreateKeyEditorParams& Params) const override;
	//~ End ISequencerChannelInterface
};

template<typename InChannelType, typename InMenuExtensionType>
void TPropertyAnimatorEditorCurveChannelInterface<InChannelType, InMenuExtensionType>::ExtendSectionMenu_Raw(FMenuBuilder& MenuBuilder
	, TSharedPtr<FExtender> InMenuExtender
	, TConstArrayView<FMovieSceneChannelHandle> InChannels
	, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
	, TWeakPtr<ISequencer> InWeakSequencer) const
{
	TSharedRef<FPropertyAnimatorEditorCurveSectionMenuExtension> Extension = MakeShared<FMenuExtensionType>(InChannels, InWeakSections);

	InMenuExtender->AddMenuExtension(TEXT("SequencerChannels"), EExtensionHook::First, nullptr
		, FMenuExtensionDelegate::CreateLambda([Extension](FMenuBuilder& InInnerMenuBuilder)
			{
				Extension->ExtendMenu(InInnerMenuBuilder, true);
			}));
}

template<typename InChannelType, typename InMenuExtensionType>
TSharedPtr<ISidebarChannelExtension> TPropertyAnimatorEditorCurveChannelInterface<InChannelType, InMenuExtensionType>::ExtendSidebarMenu_Raw(FMenuBuilder& MenuBuilder
	, TSharedPtr<FExtender> InMenuExtender
	, TConstArrayView<FMovieSceneChannelHandle> InChannels
	, const TArray<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections
	, TWeakPtr<ISequencer> InWeakSequencer) const
{
	const TSharedRef<FMenuExtensionType> Extension = MakeShared<FMenuExtensionType>(InChannels, InWeakSections);
	Extension->ExtendMenu(MenuBuilder, false);
	return Extension;
}

template<typename InChannelType, typename InMenuExtensionType>
int32 TPropertyAnimatorEditorCurveChannelInterface<InChannelType, InMenuExtensionType>::DrawExtra_Raw(FMovieSceneChannel* InChannel
	, const UMovieSceneSection* InOwner
	, const FSequencerChannelPaintArgs& InPaintArgs
	, int32 InLayerId) const
{
	if (!InOwner)
	{
		return InLayerId;
	}

	using namespace UE::Sequencer;

	FLinearColor FillColor(1, 1, 1, 0.334f);

	TArray<FVector2D> CurvePoints;
	CurvePoints.Reserve(InPaintArgs.Geometry.Size.X / 2.0);

	const FChannelType* Channel = static_cast<const FChannelType*>(InChannel);

	const double Amplitude   = Channel->Parameters.Amplitude;
	const double YScale      = (Amplitude != 0) ? (InPaintArgs.Geometry.Size.Y / Amplitude / 2.0) : 1;
	const double YOffset     = (InPaintArgs.Geometry.Size.Y * 0.5) + (Channel->Parameters.OffsetY * YScale);
	const double BaseSeconds = FPropertyAnimatorMovieSceneUtils::GetBaseSeconds(*InOwner);

	for (float X = 0.f; X < InPaintArgs.Geometry.Size.X; X += 2.f)
	{
		double Seconds = InPaintArgs.TimeToPixel.PixelToSeconds(X);
		double Y = Channel->Evaluate(BaseSeconds, Seconds);
		CurvePoints.Add(FVector2D(X, YOffset - (Y * YScale)));
	}

	FSlateDrawElement::MakeLines(InPaintArgs.DrawElements
		, InLayerId
		, InPaintArgs.Geometry.ToPaintGeometry()
		, CurvePoints
		, ESlateDrawEffect::PreMultipliedAlpha
		, FillColor
		, /*bAntialias*/true);

	return InLayerId + 1;
}

template<typename InChannelType, typename InMenuExtensionType>
TSharedRef<SWidget> TPropertyAnimatorEditorCurveChannelInterface<InChannelType, InMenuExtensionType>::CreateKeyEditor_Raw(const FMovieSceneChannelHandle& InChannel
	, const UE::Sequencer::FCreateKeyEditorParams& Params) const
{
	const TMovieSceneExternalValue<FCurveValueType>* ExternalValue = InChannel.Cast<FChannelType>().GetExtendedEditorData();
	if (!ExternalValue)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SKeyEditorWidget, FKeyEditor(Params.ObjectBindingID
		, InChannel.Cast<FChannelType>()
		, Params.OwningSection
		, Params.Sequencer
		, Params.PropertyBindings
		, ExternalValue->OnGetExternalValue));
}
