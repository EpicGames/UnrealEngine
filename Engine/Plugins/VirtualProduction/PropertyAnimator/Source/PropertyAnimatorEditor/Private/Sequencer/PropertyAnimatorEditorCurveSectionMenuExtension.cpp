// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyAnimatorEditorCurveSectionMenuExtension.h"
#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelOverrideContainer.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "DetailsViewArgs.h"
#include "Editor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneSection.h"
#include "PropertyEditorModule.h"
#include "UObject/StructOnScope.h"

#define LOCTEXT_NAMESPACE "PropertyAnimatorEditorCurveSectionMenuExtension"

void FPropertyAnimatorEditorCurveSectionMenuExtension::FChannelNotifyHook::NotifyPreChange(FProperty* InPropertyAboutToChange)
{
	if (UObject* ObjectToModify = ObjectToModifyWeak.Get())
	{
		TransactionIndex = GEditor->BeginTransaction(FText::Format(LOCTEXT("EditProperty", "Edit {0}"), InPropertyAboutToChange->GetDisplayNameText()));
		ObjectToModify->Modify();
	}
	else
	{
		TransactionIndex = INDEX_NONE;
	}
}

void FPropertyAnimatorEditorCurveSectionMenuExtension::FChannelNotifyHook::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent
	, FProperty* InPropertyThatChanged)
{
	if (TransactionIndex != INDEX_NONE)
	{
		GEditor->EndTransaction();	
	}
}

FPropertyAnimatorEditorCurveSectionMenuExtension::FPropertyAnimatorEditorCurveSectionMenuExtension(TConstArrayView<FMovieSceneChannelHandle> InChannelHandles, const TConstArrayView<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections)
	: ChannelHandles(InChannelHandles)
	, WeakSections(InWeakSections)
{
	Initialize();
}

TSharedPtr<ISidebarChannelExtension> FPropertyAnimatorEditorCurveSectionMenuExtension::ExtendMenu(FMenuBuilder& InMenuBuilder, const bool bInSubMenu)
{
	TSharedRef<FPropertyAnimatorEditorCurveSectionMenuExtension> This = SharedThis(this);

	FText MenuTitle   = LOCTEXT("ChannelsMenuLabel", "Curve Channels");
	FText MenuTooltip = LOCTEXT("ChannelsMenuTooltip", "Edit parameters for curve channels");

	if (ChannelHandles.Num() > 1)
	{
		if (bInSubMenu)
		{
			InMenuBuilder.AddSubMenu(MenuTitle, MenuTooltip, FNewMenuDelegate::CreateLambda([This](FMenuBuilder& InInnerMenuBuilder)
				{
					This->BuildChannelsMenu(InInnerMenuBuilder);
				}));
		}
		else
		{
			This->BuildChannelsMenu(InMenuBuilder);
		}
	}
	else if (ChannelHandles.Num() == 1)
	{
		if (bInSubMenu)
		{
			InMenuBuilder.AddSubMenu(MenuTitle, MenuTooltip, FNewMenuDelegate::CreateLambda([This](FMenuBuilder& InInnerMenuBuilder)
				{
					This->BuildParametersMenu(InInnerMenuBuilder, 0);
				}));
		}
		else
		{
			This->BuildParametersMenu(InMenuBuilder, 0);
		}
	}

	return This;
}

void FPropertyAnimatorEditorCurveSectionMenuExtension::Initialize()
{
	// Figure out which channels belong to which section by building the index indirections.
	// Also, create the notify hooks. Normal channels need to modify the section, but overriden channels
	// need to modify their override channel container.
	TArray<FMovieSceneChannelProxy*> ChannelProxies;
	for (const TWeakObjectPtr<UMovieSceneSection>& WeakSection : WeakSections)
	{
		ChannelProxies.Add(&WeakSection->GetChannelProxy());
	}

	for (const FMovieSceneChannelHandle& ChannelHandle : ChannelHandles)
	{
		int32 SectionIndex = ChannelProxies.Find(ChannelHandle.GetChannelProxy());
		ChannelHandleSectionIndexes.Add(SectionIndex);

		TWeakObjectPtr<UMovieSceneSection> WeakSection = WeakSections[SectionIndex];
		if (!WeakSection.IsValid())
		{
			continue;
		}

		UObject* ObjectToModify = WeakSection.Get();

		IMovieSceneChannelOverrideProvider* ChannelOverrideProvider = Cast<IMovieSceneChannelOverrideProvider>(WeakSection);

		UMovieSceneSectionChannelOverrideRegistry* ChannelOverrideRegistry = ChannelOverrideProvider
			? ChannelOverrideProvider->GetChannelOverrideRegistry(false)
			: nullptr;

		if (ChannelOverrideRegistry)
		{
			const FName ChannelName = ChannelHandle.GetMetaData()->Name;
			UMovieSceneChannelOverrideContainer* ChannelOverrideContainer = ChannelOverrideRegistry->GetChannel(ChannelName);
			if (ChannelOverrideContainer)
			{
				ensureMsgf(ChannelOverrideContainer->GetChannel() == ChannelHandle.Get(), TEXT("Mismatched channel override!"));
				ObjectToModify = ChannelOverrideContainer;
			}
		}

		NotifyHooks.Add(FChannelNotifyHook(ObjectToModify));
	}
}

void FPropertyAnimatorEditorCurveSectionMenuExtension::BuildChannelsMenu(FMenuBuilder& InMenuBuilder)
{
	const bool bMultipleSections = WeakSections.Num() > 1;

	TSharedRef<FPropertyAnimatorEditorCurveSectionMenuExtension> This = SharedThis(this);

	FText SectionMenuTooltip = LOCTEXT("SectionMenuTooltip", "Edit parameters for curve channels");

	for (int32 Index = 0; Index < ChannelHandles.Num(); ++Index)
	{
		const int32 SectionIndex(ChannelHandleSectionIndexes[Index]);

		const FMovieSceneChannelHandle ChannelHandle(ChannelHandles[Index]);

		if (bMultipleSections)
		{
			InMenuBuilder.AddSubMenu(FText::Format(LOCTEXT("ChannelAndSectionSelectMenu", "Section{0}.{1}"), SectionIndex + 1, FText::FromName(ChannelHandle.GetMetaData()->Name))
				, SectionMenuTooltip
				, FNewMenuDelegate::CreateLambda([This, Index](FMenuBuilder& InInnerMenuBuilder)
					{
						This->BuildParametersMenu(InInnerMenuBuilder, Index);
					}));
		}
		else
		{
			InMenuBuilder.AddSubMenu(FText::FromName(ChannelHandle.GetMetaData()->Name)
				, SectionMenuTooltip
				, FNewMenuDelegate::CreateLambda([This, Index](FMenuBuilder& InInnerMenuBuilder)
					{
						This->BuildParametersMenu(InInnerMenuBuilder, Index);
					}));
		}
	}
}

void FPropertyAnimatorEditorCurveSectionMenuExtension::BuildParametersMenu(FMenuBuilder& InMenuBuilder, int32 InChannelHandleIndex)
{
	if (!ensure(ChannelHandles.IsValidIndex(InChannelHandleIndex)))
	{
		return;
	}

	FMovieSceneChannelHandle ChannelHandle(ChannelHandles[InChannelHandleIndex]);

	UStruct* Struct;
	uint8* Memory;
	if (!GetParameterStructData(ChannelHandle, Struct, Memory))
	{
		return;
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowScrollBar = false;
	// The notify hook is owned by us. We will live as long as the menu is active, so as long as the NotifyHooks array isn't
	// modified, the address of the hooks should be valid.
	DetailsViewArgs.NotifyHook = &NotifyHooks[InChannelHandleIndex];

	FStructureDetailsViewArgs StructureDetailsViewArgs;
	StructureDetailsViewArgs.bShowObjects = true;
	StructureDetailsViewArgs.bShowAssets = true;
	StructureDetailsViewArgs.bShowClasses = true;
	StructureDetailsViewArgs.bShowInterfaces = true;

	TSharedRef<FStructOnScope> StructData = MakeShared<FStructOnScope>(Struct, Memory);
	TSharedRef<IStructureDetailsView> DetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs
		, StructureDetailsViewArgs
		, StructData);

	InMenuBuilder.AddWidget(DetailsView->GetWidget().ToSharedRef()
		, FText::GetEmpty()
		, /*bNoIndent*/true
		, /*bSearchable*/false);
}

#undef LOCTEXT_NAMESPACE
