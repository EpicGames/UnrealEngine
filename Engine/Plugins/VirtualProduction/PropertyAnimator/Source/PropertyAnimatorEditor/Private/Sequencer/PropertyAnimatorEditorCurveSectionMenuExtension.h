// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISequencerChannelInterface.h"
#include "Channels/MovieSceneChannelHandle.h"
#include "Containers/ArrayView.h"
#include "Misc/NotifyHook.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"

class FMenuBuilder;
class UMovieSceneSection;

class FPropertyAnimatorEditorCurveSectionMenuExtension : public TSharedFromThis<FPropertyAnimatorEditorCurveSectionMenuExtension>, public ISidebarChannelExtension
{
	struct FChannelNotifyHook : FNotifyHook
	{
		explicit FChannelNotifyHook(UObject* InObjectToModify)
			: ObjectToModifyWeak(InObjectToModify)
		{
		}

		//~ Begin FNotifyHook
		virtual void NotifyPreChange(FProperty* InPropertyAboutToChange) override;
		virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FProperty* InPropertyThatChanged) override;
		//~ End FNotifyHook

		TWeakObjectPtr<UObject> ObjectToModifyWeak;
		int32 TransactionIndex = INDEX_NONE;
	};

public:
	FPropertyAnimatorEditorCurveSectionMenuExtension(TConstArrayView<FMovieSceneChannelHandle> InChannelHandles, const TConstArrayView<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections);
	virtual ~FPropertyAnimatorEditorCurveSectionMenuExtension() = default;

	virtual bool GetParameterStructData(FMovieSceneChannelHandle InChannelHandle, UStruct*& OutStruct, uint8*& OutData) const = 0;

	virtual TSharedPtr<ISidebarChannelExtension> ExtendMenu(FMenuBuilder& InMenuBuilder, const bool bInSubMenu) override;

private:
	void Initialize();

	void BuildChannelsMenu(FMenuBuilder& InMenuBuilder);

	void BuildParametersMenu(FMenuBuilder& InMenuBuilder, int32 InChannelHandleIndex);

	TArray<FMovieSceneChannelHandle> ChannelHandles;

	TArray<int32> ChannelHandleSectionIndexes;

	TArray<TWeakObjectPtr<UMovieSceneSection>> WeakSections;

	TArray<FChannelNotifyHook> NotifyHooks;
};

template<typename InChannelType>
class TPropertyAnimatorEditorCurveSectionMenuExtension : public FPropertyAnimatorEditorCurveSectionMenuExtension
{
public:
	TPropertyAnimatorEditorCurveSectionMenuExtension(const TConstArrayView<FMovieSceneChannelHandle>& InChannelHandles, const TConstArrayView<TWeakObjectPtr<UMovieSceneSection>>& InWeakSections)
		: FPropertyAnimatorEditorCurveSectionMenuExtension(InChannelHandles, InWeakSections)
	{
	}

	template<typename InParameterType>
	UStruct* GetParameterStruct(const InParameterType& InParameters) const
	{
		return InParameterType::StaticStruct();
	}

	virtual bool GetParameterStructData(FMovieSceneChannelHandle InChannelHandle, UStruct*& OutStruct, uint8*& OutData) const override
	{
		if (InChannelType* Channel = InChannelHandle.Cast<InChannelType>().Get())
		{
			OutStruct = GetParameterStruct(Channel->Parameters);
			OutData = reinterpret_cast<uint8*>(&Channel->Parameters);
			return true;
		}
		return false;
	}
};
