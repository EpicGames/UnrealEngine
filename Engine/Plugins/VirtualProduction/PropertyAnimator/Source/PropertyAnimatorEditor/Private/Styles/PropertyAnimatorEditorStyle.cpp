// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styles/PropertyAnimatorEditorStyle.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateStyleRegistry.h"

FPropertyAnimatorEditorStyle::FPropertyAnimatorEditorStyle()
	: FSlateStyleSet(UE_MODULE_NAME)
{
	const FVector2f Icon16x16(16.f, 16.f);

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(UE_PLUGIN_NAME);

	check(Plugin.IsValid());

	ContentRootDir = FPaths::Combine(Plugin->GetBaseDir(), TEXT("Resources"));

	Set("ClassIcon.PropertyAnimatorCurveWave",  new IMAGE_BRUSH_SVG("Animators/Bounce",    Icon16x16));
	Set("ClassIcon.PropertyAnimatorSoundWave",  new IMAGE_BRUSH("Animators/SoundWave",     Icon16x16));
	Set("ClassIcon.PropertyAnimatorTime",       new IMAGE_BRUSH_SVG("Animators/Time",      Icon16x16));
	Set("ClassIcon.PropertyAnimatorWiggle",     new IMAGE_BRUSH_SVG("Animators/Wiggle",    Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FPropertyAnimatorEditorStyle::~FPropertyAnimatorEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}