// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FPropertyAnimatorEditorStyle : public FSlateStyleSet
{
public:
	FPropertyAnimatorEditorStyle();

	virtual ~FPropertyAnimatorEditorStyle() override;

	static FPropertyAnimatorEditorStyle& Get()
	{
		static FPropertyAnimatorEditorStyle StyleSet;
		return StyleSet;
	}
};