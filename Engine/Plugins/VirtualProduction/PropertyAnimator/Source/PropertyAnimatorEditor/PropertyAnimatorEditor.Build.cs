// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class PropertyAnimatorEditor : ModuleRules
{
    public PropertyAnimatorEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PrivateIncludePaths.AddRange(
            new string[] 
            {
                System.IO.Path.Combine(GetModuleDirectory("MovieSceneTools"), "Private"),
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "Core",
                "CoreUObject",
                "Engine",
                "MovieScene",
                "MovieSceneTools",
				"Projects",
                "PropertyAnimator",
                "PropertyAnimatorCore",
                "PropertyAnimatorCoreEditor",
                "Sequencer",
                "SequencerCore",
                "Slate",
                "SlateCore",
                "ToolMenus",
                "UnrealEd"
            }
        );

        ShortName = "PropAnimEd";
    }
}